// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class Json : ModuleRules
	{
		public Json(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"RapidJSON",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"TraceLog",
				}
			);

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
	}
}
