// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dom/JsonObject.h"
#include "HAL/LowLevelMemTracker.h"
#include "ProfilingDebugging/MetadataTrace.h"
#include "Hash/xxhash.h"

// See the top comment block in JsonObject.h for details.
#if (UE_JSONOBJECT_LEGACY_STRING_KEYS == 1)
UE_DEPRECATED_MACRO(5.8, "This build uses UE_JSONOBJECT_LEGACY_STRING_KEYS=1. Define UE_JSONOBJECT_LEGACY_STRING_KEYS=0 and update your code");
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS

namespace UE::JSON::Private
{
#if !UE_JSONOBJECT_LEGACY_STRING_KEYS

	UE::FSharedString FJsonStringSet::Place(FStringView Text)
	{
		LLM_SCOPE_BYNAME(TEXT("EngineMisc/Json"));
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();
		return Strings.FindOrAdd(Text);
	}

	SIZE_T FJsonStringSet::GetAllocatedSize() const
	{
		SIZE_T Size = Strings.GetAllocatedSize();
		for (const UE::FSharedString& String : Strings)
		{
			Size += String.GetAllocatedSize();
		}
		return Size;
	}

	uint32 FJsonStringSet::FCaseSensitiveStringKeyFuncs::GetKeyHash(const UE::FSharedString& Key)
	{
		const FXxHash64 Hash = FXxHash64::HashBuffer(*Key, Key.Len());
		return GetTypeHash(Hash.Hash);
	}


	FJsonObjectSharedStringStorage::FJsonObjectSharedStringStorage(TSharedPtr<UE::JSON::Private::FJsonStringSet> StringSet)
		: StringSet(MoveTemp(StringSet))
	{
	}

	SIZE_T FJsonObjectSharedStringStorage::GetAllocatedSize() const
	{
		SIZE_T SizeBytes = 0;

		SizeBytes += Values.GetAllocatedSize();
		for (const TPair<FStringType, TSharedPtr<FJsonValue>>& KV : Values)
		{
			SizeBytes += KV.Value.IsValid() ? KV.Value->GetMemoryFootprint() : 0;
		}

		if (StringSet)
		{
			SizeBytes += StringSet->GetAllocatedSize();
		}

		return SizeBytes;
	}

	void FJsonObjectSharedStringStorage::SetField(FStringView FieldName, const TSharedPtr<FJsonValue>& Value)
	{
		Values.Add(GetOrAddStringSet()->Place(FieldName), Value);
	}

	void FJsonObjectSharedStringStorage::SetNumberField(FStringView FieldName, double Number)
	{
		SetField(FieldName, MakeShared<FJsonValueNumber>(Number));
	}

	void FJsonObjectSharedStringStorage::SetStringField(FStringView FieldName, const ANSICHAR* StringValue)
	{
		SetField(FieldName, MakeShared<TJsonValueString<UTF8CHAR>>(MoveTemp(StringValue)));
	}

	void FJsonObjectSharedStringStorage::SetStringField(FStringView FieldName, const TCHAR* StringValue)
	{
		SetField(FieldName, MakeShared<TJsonValueString<TCHAR>>(FString(StringValue)));
	}

	void FJsonObjectSharedStringStorage::SetStringField(FStringView FieldName, const UTF8CHAR* StringValue)
	{
		SetField(FieldName, MakeShared<TJsonValueString<UTF8CHAR>>(StringValue));
	}

	void FJsonObjectSharedStringStorage::SetStringField(FStringView FieldName, FString&& StringValue)
	{
		SetField(FieldName, MakeShared<TJsonValueString<TCHAR>>(MoveTemp(StringValue)));
	}

	void FJsonObjectSharedStringStorage::SetStringField(FStringView FieldName, const FString& StringValue)
	{
		SetField(FieldName, MakeShared<TJsonValueString<TCHAR>>(CopyTemp(StringValue)));
	}

	void FJsonObjectSharedStringStorage::SetStringField(FStringView FieldName, FUtf8String&& StringValue)
	{
		SetField(FieldName, MakeShared<TJsonValueString<UTF8CHAR>>(MoveTemp(StringValue)));
	}

	void FJsonObjectSharedStringStorage::SetStringField(FStringView FieldName, const FUtf8String& StringValue)
	{
		SetField(FieldName, MakeShared<TJsonValueString<UTF8CHAR>>(CopyTemp(StringValue)));
	}

	void FJsonObjectSharedStringStorage::SetStringField(FStringView FieldName, const UE::FSharedString& StringValue)
	{
		SetField(FieldName, MakeShared<TJsonValueString<TCHAR>>(FString(StringValue.ToView())));
	}

	void FJsonObjectSharedStringStorage::SetBoolField(FStringView FieldName, bool InValue)
	{
		SetField(FieldName, MakeShared<FJsonValueBoolean>(InValue));
	}

	void FJsonObjectSharedStringStorage::SetArrayField(FStringView FieldName, TArray<TSharedPtr<FJsonValue>>&& Array)
	{
		SetField(FieldName, MakeShared<FJsonValueArray>(MoveTemp(Array)));
	}

	void FJsonObjectSharedStringStorage::SetArrayField(FStringView FieldName, const TArray<TSharedPtr<FJsonValue>>& Array)
	{
		SetField(FieldName, MakeShared<FJsonValueArray>(CopyTemp(Array)));
	}

	void FJsonObjectSharedStringStorage::SetObjectField(FStringView FieldName, TSharedPtr<FJsonObject>&& JsonObject)
	{
		if (JsonObject.IsValid())
		{
			SetField(FieldName, MakeShared<FJsonValueObject>(MoveTemp(JsonObject)));
		}
		else
		{
			SetField(FieldName, MakeShared<FJsonValueNull>());
		}
	}

	void FJsonObjectSharedStringStorage::SetObjectField(FStringView FieldName, const TSharedPtr<FJsonObject>& JsonObject)
	{
		if (JsonObject.IsValid())
		{
			SetField(FieldName, MakeShared<FJsonValueObject>(CopyTemp(JsonObject)));
		}
		else
		{
			SetField(FieldName, MakeShared<FJsonValueNull>());
		}
	}

#else

	void FJsonObjectStringStorage::SetField(FStringView FieldName, const TSharedPtr<FJsonValue>& Value)
	{
		Values.Add(FString(FieldName), Value);
	}

	void FJsonObjectStringStorage::SetField(const FString& FieldName, const TSharedPtr<FJsonValue>& Value)
	{
		Values.Add(CopyTemp(FieldName), Value);
	}

	void FJsonObjectStringStorage::SetField(const TCHAR* FieldName, const TSharedPtr<FJsonValue>& Value)
	{
		Values.Add(FString(FieldName), Value);
	}

	void FJsonObjectStringStorage::SetField(const UE::FSharedString& FieldName, const TSharedPtr<FJsonValue>& Value)
	{
		Values.Add(FString(*FieldName), Value);
	}

	void FJsonObjectStringStorage::SetNumberField(FString&& FieldName, double Number)
	{
		Values.Add(MoveTemp(FieldName), MakeShared<FJsonValueNumber>(Number));
	}

	void FJsonObjectStringStorage::SetNumberField(const FString& FieldName, double Number)
	{
		SetNumberField(CopyTemp(FieldName), Number);
	}

	void FJsonObjectStringStorage::SetStringField(FString&& FieldName, const ANSICHAR* StringValue)
	{
		SetStringField(MoveTemp(FieldName), FUtf8String(StringValue));
	}

	void FJsonObjectStringStorage::SetStringField(const FString& FieldName, const ANSICHAR* StringValue)
	{
		SetStringField(FieldName, FUtf8String(StringValue));
	}

	void FJsonObjectStringStorage::SetStringField(FString&& FieldName, const TCHAR* StringValue)
	{
		SetStringField(MoveTemp(FieldName), FString(StringValue));
	}

	void FJsonObjectStringStorage::SetStringField(const FString& FieldName, const TCHAR* StringValue)
	{
		SetStringField(FieldName, FString(StringValue));
	}

	void FJsonObjectStringStorage::SetStringField(FString&& FieldName, const UTF8CHAR* StringValue)
	{
		SetStringField(MoveTemp(FieldName), FUtf8String(StringValue));
	}

	void FJsonObjectStringStorage::SetStringField(const FString& FieldName, const UTF8CHAR* StringValue)
	{
		SetStringField(FieldName, FUtf8String(StringValue));
	}

	void FJsonObjectStringStorage::SetStringField(FString&& FieldName, FString&& StringValue)
	{
		Values.Add(MoveTemp(FieldName), MakeShared<TJsonValueString<TCHAR>>(MoveTemp(StringValue)));
	}

	void FJsonObjectStringStorage::SetStringField(FString&& FieldName, const FString& StringValue)
	{
		SetStringField(MoveTemp(FieldName), CopyTemp(StringValue));
	}

	void FJsonObjectStringStorage::SetStringField(const FString& FieldName, FString&& StringValue)
	{
		SetStringField(CopyTemp(FieldName), MoveTemp(StringValue));
	}

	void FJsonObjectStringStorage::SetStringField(const FString& FieldName, const FString& StringValue)
	{
		SetStringField(CopyTemp(FieldName), CopyTemp(StringValue));
	}

	void FJsonObjectStringStorage::SetStringField(FString&& FieldName, FUtf8String&& StringValue)
	{
		Values.Add(MoveTemp(FieldName), MakeShared<TJsonValueString<UTF8CHAR>>(MoveTemp(StringValue)));
	}

	void FJsonObjectStringStorage::SetStringField(FString&& FieldName, const FUtf8String& StringValue)
	{
		SetStringField(MoveTemp(FieldName), CopyTemp(StringValue));
	}

	void FJsonObjectStringStorage::SetStringField(const FString& FieldName, FUtf8String&& StringValue)
	{
		SetStringField(CopyTemp(FieldName), MoveTemp(StringValue));
	}

	void FJsonObjectStringStorage::SetStringField(const FString& FieldName, const FUtf8String& StringValue)
	{
		SetStringField(CopyTemp(FieldName), CopyTemp(StringValue));
	}

	void FJsonObjectStringStorage::SetBoolField(FString&& FieldName, bool InValue)
	{
		Values.Add(MoveTemp(FieldName), MakeShared<FJsonValueBoolean>(InValue));
	}

	void FJsonObjectStringStorage::SetBoolField(const FString& FieldName, bool InValue)
	{
		SetBoolField(CopyTemp(FieldName), InValue);
	}

	void FJsonObjectStringStorage::SetArrayField(FString&& FieldName, TArray<TSharedPtr<FJsonValue>>&& Array)
	{
		Values.Add(MoveTemp(FieldName), MakeShared<FJsonValueArray>(MoveTemp(Array)));
	}

	void FJsonObjectStringStorage::SetArrayField(FString&& FieldName, const TArray<TSharedPtr<FJsonValue>>& Array)
	{
		SetArrayField(MoveTemp(FieldName), CopyTemp(Array));
	}

	void FJsonObjectStringStorage::SetArrayField(const FString& FieldName, TArray<TSharedPtr<FJsonValue>>&& Array)
	{
		SetArrayField(CopyTemp(FieldName), MoveTemp(Array));
	}

	void FJsonObjectStringStorage::SetArrayField(const FString& FieldName, const TArray<TSharedPtr<FJsonValue>>& Array)
	{
		SetArrayField(CopyTemp(FieldName), CopyTemp(Array));
	}

	void FJsonObjectStringStorage::SetObjectField(FString&& FieldName, TSharedPtr<FJsonObject>&& JsonObject)
	{
		if (JsonObject.IsValid())
		{
			Values.Emplace(MoveTemp(FieldName), MakeShared<FJsonValueObject>(MoveTemp(JsonObject)));
		}
		else
		{
			Values.Emplace(MoveTemp(FieldName), MakeShared<FJsonValueNull>());
		}
	}

	void FJsonObjectStringStorage::SetObjectField(FString&& FieldName, const TSharedPtr<FJsonObject>& JsonObject)
	{
		SetObjectField(MoveTemp(FieldName), TSharedPtr<FJsonObject>(JsonObject));
	}

	void FJsonObjectStringStorage::SetObjectField(const FString& FieldName, const TSharedPtr<FJsonObject>& JsonObject)
	{
		SetObjectField(CopyTemp(FieldName), JsonObject);
	}

	SIZE_T FJsonObjectStringStorage::GetAllocatedSize() const
	{
		SIZE_T SizeBytes = 0;

		SizeBytes += Values.GetAllocatedSize();
		for (const TPair<FString, TSharedPtr<FJsonValue>>& KV : Values)
		{
			SizeBytes += KV.Value.IsValid() ? KV.Value->GetMemoryFootprint() : 0;
		}

		return SizeBytes;
	}

#endif

}

const TSharedPtr<FJsonValue> FJsonObject::GetFieldUntyped(FStringView FieldName) const
{
	const TSharedPtr<FJsonValue>* Ptr = Values.FindByHash(GetTypeHash(FieldName), FieldName);
	if (Ptr != nullptr)
	{
		return *Ptr;
	}
	return {};
}

TSharedPtr<FJsonValue> FJsonObject::GetField(FStringView FieldName, EJson JsonType) const
{
	const TSharedPtr<FJsonValue>* Field = Values.FindByHash(GetTypeHash(FieldName), FieldName);
	if (Field != nullptr && Field->IsValid())
	{
		if (JsonType == EJson::None || (*Field)->Type == JsonType)
		{
			return (*Field);
		}
		else
		{
			UE_LOGF(LogJson, Warning, "Field %.*ls is of the wrong type.", FieldName.Len(), FieldName.GetData());
		}
	}
	else
	{
		UE_LOGF(LogJson, Warning, "Field %.*ls was not found.", FieldName.Len(), FieldName.GetData());
	}

	return MakeShared<FJsonValueNull>();
}

TSharedPtr<FJsonValue> FJsonObject::TryGetField(FStringView FieldName) const
{
	const TSharedPtr<FJsonValue>* Field = Values.FindByHash(GetTypeHash(FieldName), FieldName);
	return (Field != nullptr && Field->IsValid()) ? *Field : TSharedPtr<FJsonValue>();
}

bool FJsonObject::HasField(FStringView FieldName) const
{
	const TSharedPtr<FJsonValue>* Field = Values.FindByHash(GetTypeHash(FieldName), FieldName);
	if (Field && Field->IsValid())
	{
		return true;
	}

	return false;
}

bool FJsonObject::HasTypedField(FStringView FieldName, EJson JsonType) const
{
	const TSharedPtr<FJsonValue>* Field = Values.FindByHash(GetTypeHash(FieldName), FieldName);
	if (Field && Field->IsValid() && ((*Field)->Type == JsonType))
	{
		return true;
	}

	return false;
}

void FJsonObject::RemoveField(FStringView FieldName)
{
	Values.RemoveByHash(GetTypeHash(FieldName), FieldName);
}

void FJsonObject::RemoveIf(TFunctionRef<bool(const FStringType&, TSharedPtr<const FJsonValue>)> Predicate)
{
	for (typename TMap<FStringType, TSharedPtr<FJsonValue>>::TIterator It(Values); It; ++It)
	{
		if (Predicate(It.Key(), It.Value()))
		{
			It.RemoveCurrent();
		}
	}
}

double FJsonObject::GetNumberField(FStringView FieldName) const
{
	return GetField<EJson::None>(FieldName)->AsNumber();
}

int32 FJsonObject::GetIntegerField(FStringView FieldName) const
{
	return (int32)GetNumberField(FieldName);
}

bool FJsonObject::TryGetNumberField(FStringView FieldName, float& OutNumber) const
{
	TSharedPtr<FJsonValue> Field = TryGetField(FieldName);
	return Field.IsValid() && Field->TryGetNumber(OutNumber);
}

bool FJsonObject::TryGetNumberField(FStringView FieldName, double& OutNumber) const
{
	TSharedPtr<FJsonValue> Field = TryGetField(FieldName);
	return Field.IsValid() && Field->TryGetNumber(OutNumber);
}

bool FJsonObject::TryGetNumberField(FStringView FieldName, int8& OutNumber) const
{
	TSharedPtr<FJsonValue> Field = TryGetField(FieldName);
	return Field.IsValid() && Field->TryGetNumber(OutNumber);
}

bool FJsonObject::TryGetNumberField(FStringView FieldName, int16& OutNumber) const
{
	TSharedPtr<FJsonValue> Field = TryGetField(FieldName);
	return Field.IsValid() && Field->TryGetNumber(OutNumber);
}

bool FJsonObject::TryGetNumberField(FStringView FieldName, int32& OutNumber) const
{
	TSharedPtr<FJsonValue> Field = TryGetField(FieldName);
	return Field.IsValid() && Field->TryGetNumber(OutNumber);
}

bool FJsonObject::TryGetNumberField(FStringView FieldName, int64& OutNumber) const
{
	TSharedPtr<FJsonValue> Field = TryGetField(FieldName);
	return Field.IsValid() && Field->TryGetNumber(OutNumber);
}

bool FJsonObject::TryGetNumberField(FStringView FieldName, uint8& OutNumber) const
{
	TSharedPtr<FJsonValue> Field = TryGetField(FieldName);
	return Field.IsValid() && Field->TryGetNumber(OutNumber);
}

bool FJsonObject::TryGetNumberField(FStringView FieldName, uint16& OutNumber) const
{
	TSharedPtr<FJsonValue> Field = TryGetField(FieldName);
	return Field.IsValid() && Field->TryGetNumber(OutNumber);
}

bool FJsonObject::TryGetNumberField(FStringView FieldName, uint32& OutNumber) const
{
	TSharedPtr<FJsonValue> Field = TryGetField(FieldName);
	return Field.IsValid() && Field->TryGetNumber(OutNumber);
}

bool FJsonObject::TryGetNumberField(FStringView FieldName, uint64& OutNumber) const
{
	TSharedPtr<FJsonValue> Field = TryGetField(FieldName);
	return Field.IsValid() && Field->TryGetNumber(OutNumber);
}

FString FJsonObject::GetStringField(FStringView FieldName) const
{
	return GetField<EJson::None>(FieldName)->AsString();
}

FUtf8String FJsonObject::GetUtf8StringField(FStringView FieldName) const
{
	return GetField<EJson::None>(FieldName)->AsUtf8String();
}

bool FJsonObject::TryGetStringField(FStringView FieldName, FString& OutString) const
{
	TSharedPtr<FJsonValue> Field = TryGetField(FieldName);
	return Field.IsValid() && Field->TryGetString(OutString);
}

bool FJsonObject::TryGetStringArrayField(FStringView FieldName, TArray<FString>& OutArray) const
{
	TSharedPtr<FJsonValue> Field = TryGetField(FieldName);

	if (!Field.IsValid())
	{
		return false;
	}

	const TArray< TSharedPtr<FJsonValue> >* Array;

	if (!Field->TryGetArray(Array))
	{
		return false;
	}

	for (int Idx = 0; Idx < Array->Num(); Idx++)
	{
		FString Element;

		if (!(*Array)[Idx]->TryGetString(Element))
		{
			return false;
		}

		OutArray.Add(Element);
	}

	return true;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool FJsonObject::GetBoolField(FStringView FieldName) const
{
	return GetField<EJson::None>(FieldName)->AsBool();
}

bool FJsonObject::TryGetBoolField(FStringView FieldName, bool& OutBool) const
{
	TSharedPtr<FJsonValue> Field = TryGetField(FieldName);
	return Field.IsValid() && Field->TryGetBool(OutBool);
}

const TArray<TSharedPtr<FJsonValue>>& FJsonObject::GetArrayField(FStringView FieldName) const
{
	return GetField<EJson::Array>(FieldName)->AsArray();
}

bool FJsonObject::TryGetArrayField(FStringView FieldName, const TArray<TSharedPtr<FJsonValue>>*& OutArray) const
{
	TSharedPtr<FJsonValue> Field = TryGetField(FieldName);
	return Field.IsValid() && Field->TryGetArray(OutArray);
}

const TSharedPtr<FJsonObject>& FJsonObject::GetObjectField(FStringView FieldName) const
{
	return GetField<EJson::Object>(FieldName)->AsObject();
}

bool FJsonObject::TryGetObjectField(FStringView FieldName, const TSharedPtr<FJsonObject>*& OutObject) const
{
	TSharedPtr<FJsonValue> Field = TryGetField(FieldName);
	return Field.IsValid() && Field->TryGetObject(OutObject);
}

void FJsonObject::Duplicate(const TSharedPtr<const FJsonObject>& Source, const TSharedPtr<FJsonObject>& Dest)
{
	if (Source && Dest)
	{
		for (const TPair<FStringType, TSharedPtr<FJsonValue>>& KV : Source->Values)
		{
			Dest->SetField(KV.Key, FJsonValue::Duplicate(KV.Value));
		}
	}
}

void FJsonObject::Duplicate(const TSharedPtr<FJsonObject>& Source, TSharedPtr<FJsonObject>& Dest)
{
	Duplicate(ConstCastSharedPtr<const FJsonObject>(Source), Dest);
}
