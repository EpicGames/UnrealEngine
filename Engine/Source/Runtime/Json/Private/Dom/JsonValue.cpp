// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"

FJsonValue::FJsonValue()
: Type(EJson::None) 
{
}

FJsonValue::~FJsonValue() = default;

double FJsonValue::AsNumber() const
{
	double Number = 0.0;

	if (!TryGetNumber(Number))
	{
		ErrorMessage(TEXT("Number"));
	}

	return Number;
}


FString FJsonValue::AsString() const 
{
	FString String;

	if (!TryGetString(String))
	{
		ErrorMessage(TEXT("String"));
	}

	return String;
}

FUtf8String FJsonValue::AsUtf8String() const
{
	FUtf8String String;

	if (!TryGetUtf8String(String))
	{
		ErrorMessage(TEXT("Utf8String"));
	}

	return String;
}

bool FJsonValue::AsBool() const 
{
	bool Bool = false;

	if (!TryGetBool(Bool))
	{
		ErrorMessage(TEXT("Boolean")); 
	}

	return Bool;
}


const TArray<TSharedPtr<FJsonValue>>& FJsonValue::AsArray() const
{
	const TArray<TSharedPtr<FJsonValue>>* Array = nullptr;

	if (!TryGetArray(Array))
	{
		static const TArray<TSharedPtr<FJsonValue>> EmptyArray;
		Array = &EmptyArray;
		ErrorMessage(TEXT("Array"));
	}

	return *Array;
}


const TSharedPtr<FJsonObject>& FJsonValue::AsObject() const
{
	const TSharedPtr<FJsonObject>* Object = nullptr;

	if (!TryGetObject(Object))
	{
		static const TSharedPtr<FJsonObject> EmptyObject = MakeShared<FJsonObject>();
		Object = &EmptyObject;
		ErrorMessage(TEXT("Object"));
	}

	return *Object;
}

// -----------------------------------

template <typename T>
bool TryConvertNumber(const FJsonValue& InValue, T& OutNumber)
{
	double Double;

	if (InValue.TryGetNumber(Double) && (Double >= TNumericLimits<T>::Min()) && (Double <= static_cast<double>(TNumericLimits<T>::Max())))
	{
		OutNumber = static_cast<T>(FMath::RoundHalfFromZero(Double));

		return true;
	}

	return false;
}

// Need special handling for int64/uint64, due to overflow in the numeric limits.
// 2^63-1 and 2^64-1 cannot be exactly represented as a double, so TNumericLimits<>::Max() gets rounded up to exactly 2^63 or 2^64 by the compiler's implicit cast to double.
// This breaks the overflow check in TryConvertNumber. We use "<" rather than "<=" along with the exact power-of-two double literal to fix this.
template <> 
bool TryConvertNumber<uint64>(const FJsonValue& InValue, uint64& OutNumber)
{
	double Double;
	if (InValue.TryGetNumber(Double) && Double >= 0.0 && Double < 18446744073709551616.0)
	{
		OutNumber = static_cast<uint64>(FMath::RoundHalfFromZero(Double));
		return true;
	}

	return false;
}

template <>
bool TryConvertNumber<int64>(const FJsonValue& InValue, int64& OutNumber)
{
	double Double;
	if (InValue.TryGetNumber(Double) && Double >= -9223372036854775808.0 && Double < 9223372036854775808.0)
	{
		OutNumber = static_cast<int64>(FMath::RoundHalfFromZero(Double));
		return true;
	}

	return false;
}

// -----------------------------------
bool FJsonValue::TryGetNumber(double& OutNumber) const
{
	return false;
}

bool FJsonValue::TryGetNumber(float& OutNumber) const
{
	double Double;

	if (TryGetNumber(Double))
	{
		OutNumber = static_cast<float>(Double);
		return true;
	}

	return false;
}

bool FJsonValue::TryGetNumber(uint8& OutNumber) const
{
	return TryConvertNumber(*this, OutNumber);
}

bool FJsonValue::TryGetNumber(uint16& OutNumber) const
{
	return TryConvertNumber(*this, OutNumber);
}

bool FJsonValue::TryGetNumber(uint32& OutNumber) const
{
	return TryConvertNumber(*this, OutNumber);
}

bool FJsonValue::TryGetNumber(uint64& OutNumber) const
{
	return TryConvertNumber(*this, OutNumber);
}

bool FJsonValue::TryGetNumber(int8& OutNumber) const
{
	return TryConvertNumber(*this, OutNumber);
}

bool FJsonValue::TryGetNumber(int16& OutNumber) const
{
	return TryConvertNumber(*this, OutNumber);
}

bool FJsonValue::TryGetNumber(int32& OutNumber) const
{
	return TryConvertNumber(*this, OutNumber);
}

bool FJsonValue::TryGetNumber(int64& OutNumber) const
{
	return TryConvertNumber(*this, OutNumber);
}

bool FJsonValue::TryGetString(FString& OutString) const
{
	return false;
}

bool FJsonValue::TryGetUtf8String(FUtf8String& OutString) const
{
	return false;
}

bool FJsonValue::TryGetBool(bool& OutBool) const
{
	return false;
}

bool FJsonValue::TryGetArray(const TArray<TSharedPtr<FJsonValue>>*& OutArray) const
{
	return false;
}
	
bool FJsonValue::TryGetArray(TArray<TSharedPtr<FJsonValue>>*& OutArray)
{
	return false;
}

bool FJsonValue::TryGetObject(const TSharedPtr<FJsonObject>*& Object) const
{
	return false;
}

bool FJsonValue::TryGetObject(TSharedPtr<FJsonObject>*& Object)
{
	return false;
}

bool FJsonValue::PreferStringRepresentation() const
{
	return false;
}

SIZE_T FJsonValue::GetMemoryFootprint() const
{
	return sizeof(*this);
}

//static 
bool FJsonValue::CompareEqual( const FJsonValue& Lhs, const FJsonValue& Rhs )
{
	if (Lhs.Type != Rhs.Type)
	{
		const bool bLhsIsSimpleVariant = Lhs.Type == EJson::Boolean || Lhs.Type == EJson::Number || Lhs.Type == EJson::String;
		const bool bRhsIsSimpleVariant = Rhs.Type == EJson::Boolean || Rhs.Type == EJson::Number || Rhs.Type == EJson::String;
		if (bLhsIsSimpleVariant && bRhsIsSimpleVariant)
		{
			return UE::Json::ToSimpleJsonVariant(Lhs) == UE::Json::ToSimpleJsonVariant(Rhs);
		}
		else
		{
			return false;
		}
	}

	switch (Lhs.Type)
	{
	case EJson::None:
	case EJson::Null:
		return true;

	case EJson::String:
		return Lhs.AsString() == Rhs.AsString();

	case EJson::Number:
		return Lhs.AsNumber() == Rhs.AsNumber();

	case EJson::Boolean:
		return Lhs.AsBool() == Rhs.AsBool();

	case EJson::Array:
		{
			const TArray<TSharedPtr<FJsonValue>>& LhsArray = Lhs.AsArray();
			const TArray<TSharedPtr<FJsonValue>>& RhsArray = Rhs.AsArray();

			if (LhsArray.Num() != RhsArray.Num())
			{
				return false;
			}

			// compare each element
			for (int32 i = 0; i < LhsArray.Num(); ++i)
			{
				if (!CompareEqual(*LhsArray[i], *RhsArray[i]))
				{
					return false;
				}
			}
		}
		return true;

	case EJson::Object:
		{
			const TSharedPtr<FJsonObject>& LhsObject = Lhs.AsObject();
			const TSharedPtr<FJsonObject>& RhsObject = Rhs.AsObject();

			if (LhsObject.IsValid() != RhsObject.IsValid())
			{
				return false;
			}

			if (LhsObject.IsValid())
			{
				if (LhsObject->Values.Num() != RhsObject->Values.Num())
				{
					return false;
				}

				// compare each element
				for (const auto& It : LhsObject->Values)
				{
					const FJsonObject::FStringType& Key = It.Key;
					const TSharedPtr<FJsonValue> RhsValue = RhsObject->GetFieldUntyped(Key);
					if (!RhsValue.IsValid())
					{
						// not found in both objects
						return false;
					}

					const TSharedPtr<FJsonValue>& LhsValue = It.Value;

					if (LhsValue.IsValid() != RhsValue.IsValid())
					{
						return false;
					}

					if (LhsValue.IsValid())
					{
						if (!CompareEqual(*LhsValue, *RhsValue))
						{
							return false;
						}
					}
				}
			}
		}
		return true;

	default:
		return false;
	}
}

static void DuplicateJsonArray(const TArray<TSharedPtr<FJsonValue>>& Source, TArray<TSharedPtr<FJsonValue>>& Dest)
{
	for (const TSharedPtr<FJsonValue>& Value : Source)
	{
		Dest.Add(FJsonValue::Duplicate(Value));
	} 
}

TSharedPtr<FJsonValue> FJsonValue::Duplicate(const TSharedPtr<const FJsonValue>& Src)
{
	return Duplicate(ConstCastSharedPtr<FJsonValue>(Src));
}

TSharedPtr<FJsonValue> FJsonValue::Duplicate(const TSharedPtr<FJsonValue>& Src)
{
	switch (Src->Type)
	{
		case EJson::Boolean:
		{
			bool BoolValue;
			if (Src->TryGetBool(BoolValue))
			{
				return MakeShared<FJsonValueBoolean>(BoolValue);
			}
		}
		case EJson::Number:
		{
			double NumberValue;
			if (Src->TryGetNumber(NumberValue))
			{
				return MakeShared<FJsonValueNumber>(NumberValue);
			}
		}
		case EJson::String:
		{
			FString StringValue;
			if (Src->TryGetString(StringValue))
			{
				return MakeShared<FJsonValueString>(StringValue);
			}
		}
		case EJson::Object:
		{
			const TSharedPtr<FJsonObject>* ObjectValue;
			if (Src->TryGetObject(ObjectValue))
			{
				TSharedPtr<FJsonObject> NewObject = MakeShared<FJsonObject>();
				FJsonObject::Duplicate(*ObjectValue, NewObject);
				return MakeShared<FJsonValueObject>(NewObject);
			}
		}
		case EJson::Array:
		{
			const TArray<TSharedPtr<FJsonValue>>* ArrayValue;
			if (Src->TryGetArray(ArrayValue))
			{
				TArray<TSharedPtr<FJsonValue>> NewArray;
				DuplicateJsonArray(*ArrayValue, NewArray);

				return MakeShared<FJsonValueArray>(NewArray);
			}
		}
	}

	return TSharedPtr<FJsonValue>();
}

void FJsonValue::ErrorMessage(const FString& InType) const
{
	if (IsNull())
	{
		UE_LOGF(LogJson, Warning, "Json Value of type '%ls' used as a '%ls'.", *GetType(), *InType);
	}
	else
	{
		UE_LOGF(LogJson, Error, "Json Value of type '%ls' used as a '%ls'.", *GetType(), *InType);
	}
}

FJsonValueNumber::FJsonValueNumber(double InNumber)
: Value(InNumber)
{
	Type = EJson::Number;
}

FJsonValueNumber::~FJsonValueNumber() = default;

bool FJsonValueNumber::TryGetNumber(double& OutNumber) const
{
	OutNumber = Value;
	return true;
}

bool FJsonValueNumber::TryGetBool(bool& OutBool) const
{
	OutBool = (Value != 0.0);
	return true;
}

bool FJsonValueNumber::TryGetString(FString& OutString) const
{
	OutString = FString::SanitizeFloat(Value, 0);
	return true;
}

bool FJsonValueNumber::TryGetUtf8String(FUtf8String& OutString) const
{
	OutString = FUtf8String::SanitizeFloat(Value, 0);
	return true;
}

SIZE_T FJsonValueNumber::GetMemoryFootprint() const
{
	return sizeof(*this);
}

FString FJsonValueNumber::GetType() const
{
	return TEXT("Number");
}

FJsonValueBoolean::FJsonValueBoolean(bool InBool)
: Value(InBool) 
{
	Type = EJson::Boolean;
}

FJsonValueBoolean::~FJsonValueBoolean() = default;

bool FJsonValueBoolean::TryGetNumber(double& OutNumber) const
{
	OutNumber = Value ? 1 : 0;
	return true;
}

bool FJsonValueBoolean::TryGetBool(bool& OutBool) const
{
	OutBool = Value;
	return true;
}

bool FJsonValueBoolean::TryGetString(FString& OutString) const
{
	OutString = Value ? TEXT("true") : TEXT("false");
	return true;
}

bool FJsonValueBoolean::TryGetUtf8String(FUtf8String& OutString) const
{
	OutString = Value ? UTF8TEXT("true") : UTF8TEXT("false");
	return true;
}

SIZE_T FJsonValueBoolean::GetMemoryFootprint() const
{
	return sizeof(*this);
}

FString FJsonValueBoolean::GetType() const
{
	return TEXT("Boolean");
}

FJsonValueArray::FJsonValueArray(const TArray<TSharedPtr<FJsonValue>>& InArray)
: Value(InArray)
{
	Type = EJson::Array;
}

FJsonValueArray::FJsonValueArray(TArray<TSharedPtr<FJsonValue>>&& InArray)
: Value(MoveTemp(InArray))
{
	Type = EJson::Array;
}

FJsonValueArray::~FJsonValueArray() = default;

bool FJsonValueArray::TryGetArray(const TArray<TSharedPtr<FJsonValue>>*& OutArray) const
{
	OutArray = &Value;
	return true;
}

bool FJsonValueArray::TryGetArray(TArray<TSharedPtr<FJsonValue>>*& OutArray)
{
	OutArray = &Value;
	return true;
}

SIZE_T FJsonValueArray::GetMemoryFootprint() const
{
	 return sizeof(*this) + GetAllocatedSize(); 
}

SIZE_T FJsonValueArray::GetAllocatedSize() const
{
	SIZE_T SizeBytes = 0;
	SizeBytes += Value.GetAllocatedSize();
	for (const TSharedPtr<FJsonValue>& Element : Value)
	{
		SizeBytes += Element.IsValid() ? Element->GetMemoryFootprint() : 0;
	}
	return SizeBytes;
}

FString FJsonValueArray::GetType() const
{
	return TEXT("Array");
}	

FJsonValueObject::FJsonValueObject(TSharedPtr<FJsonObject> InObject)
: Value(MoveTemp(InObject))
{
	Type = EJson::Object;
}

FJsonValueObject::~FJsonValueObject() = default;

bool FJsonValueObject::TryGetObject(const TSharedPtr<FJsonObject>*& OutObject) const
{
	OutObject = &Value;
	return true;
}

bool FJsonValueObject::TryGetObject(TSharedPtr<FJsonObject>*& OutObject)
{
	OutObject = &Value;
	return true;
}

SIZE_T FJsonValueObject::GetMemoryFootprint() const
{
	return sizeof(*this) + GetAllocatedSize();
}

SIZE_T FJsonValueObject::GetAllocatedSize() const
{
	return Value.IsValid() ? Value->GetMemoryFootprint() : 0;
}

FString FJsonValueObject::GetType() const
{
	return TEXT("Object");
}

FJsonValueNull::FJsonValueNull()
{
	Type = EJson::Null;
}

FJsonValueNull::~FJsonValueNull() = default;

SIZE_T FJsonValueNull::GetMemoryFootprint() const
{
	return sizeof(*this);
}

FString FJsonValueNull::GetType() const
{
	return TEXT("Null");
}

namespace UE::Json
{
	JsonSimpleValueVariant ToSimpleJsonVariant(const FJsonValue& InJsonValue)
	{
		if (!InJsonValue.PreferStringRepresentation())
		{
			if (InJsonValue.Type == EJson::Boolean)
			{
				return JsonSimpleValueVariant(TInPlaceType<bool>(), InJsonValue.AsBool());
			}
			else if (InJsonValue.Type == EJson::Number)
			{
				const double JsonNumber_v = InJsonValue.AsNumber();

				/* If the Json Number Value requires a decimal point, then we read in the value as a double, otherwise, we read it in as an int */
				if (FString::SanitizeFloat(JsonNumber_v, 0).Contains(TEXT(".")))
				{
					return ToSimpleJsonVariant(JsonNumber_v);
				}
				else
				{
					return ToSimpleJsonVariant(FMath::RoundToInt64(JsonNumber_v));
				}
			}
		}

		return JsonSimpleValueVariant(TInPlaceType<FString>(), InJsonValue.AsString());
	}
}

bool operator==(const JsonNumberValueVariants& Lhs, const FString& Rhs)
{
	return Rhs.IsNumeric() && ::Visit([Rhs](const auto& StoredNumber)
		{
			using StoredNumberType = std::decay_t<decltype(StoredNumber)>;
			if constexpr (std::is_same_v<StoredNumberType, float> || std::is_same_v<StoredNumberType, double>)
			{
				return FString::SanitizeFloat(StoredNumber, 0) == Rhs;
			}
			else
			{
				return StoredNumber == FCString::Atoi64(*Rhs);
			}
		}, Lhs);
}


FString ToString(const JsonNumberValueVariants& InNumberVariant)
{
	return ::Visit([](auto& StoredNumber)
		{
			using StoredNumberType = std::decay_t<decltype(StoredNumber)>;
			if constexpr (std::is_same_v<StoredNumberType, float> || std::is_same_v<StoredNumberType, double>)
			{
				return FString::SanitizeFloat(StoredNumber, 0);
			}
			else
			{
				return FString::Printf(TEXT("%lld"), static_cast<int64>(StoredNumber));
			}
		}, InNumberVariant);
}

bool operator==(const JsonNumberValueVariants& Lhs, const JsonNumberValueVariants& Rhs)
{
	const bool bLhsIsFloat = Lhs.IsType<float>() || Lhs.IsType<double>();
	const bool bRhsIsFloat = Rhs.IsType<float>() || Rhs.IsType<double>();
	if (bLhsIsFloat || bRhsIsFloat)
	{
		return ToString(Lhs) == ToString(Rhs);
	}
	else
	{
		auto CastToInt64Functor = [](auto& StoredNumber)
			{
				return static_cast<int64>(StoredNumber);
			};

		const int64 LhsValue = ::Visit(CastToInt64Functor, Lhs);
		const int64 RhsValue = ::Visit(CastToInt64Functor, Rhs);

		return LhsValue == RhsValue;
	}
}


bool operator==(const JsonSimpleValueVariant& Lhs, const JsonSimpleValueVariant& Rhs)
{
	if (Lhs.IsType<bool>())
	{
		if (Rhs.IsType<bool>())
		{
			return Lhs.Get<bool>() == Rhs.Get<bool>();
		}
		else if (Rhs.IsType<FString>())
		{
			if (Lhs.Get<bool>())
			{
				return Rhs.Get<FString>().Equals(TEXT("true"), ESearchCase::IgnoreCase) ||
					Rhs.Get<FString>().Equals(TEXT("1"), ESearchCase::IgnoreCase);
			}
			else
			{
				return Rhs.Get<FString>().Equals(TEXT("false"), ESearchCase::IgnoreCase) ||
					Rhs.Get<FString>().Equals(TEXT("0"), ESearchCase::IgnoreCase);
			}
		}
		else // RhsType.IsType<JsonNumberValueVariants>()
		{
			return ::Visit([&Lhs](const auto& RhsStoredNumber)
				{
					using RhsStoredNumberType = std::decay_t<decltype(RhsStoredNumber)>;
					if constexpr (std::is_same_v<RhsStoredNumberType, float> || std::is_same_v<RhsStoredNumberType, double>)
					{
						if (!FString::SanitizeFloat(RhsStoredNumber, 0).Contains(TEXT(".")))
						{
							const int64 RhsStoredNumberAsInt = FMath::RoundToInt64(RhsStoredNumber);
							if (Lhs.Get<bool>())
							{
								return RhsStoredNumberAsInt == 1;
							}
							else
							{
								return RhsStoredNumberAsInt == 0;
							}
						}
						else
						{
							return false;
						}
					}
					else
					{
						if (Lhs.Get<bool>())
						{
							return RhsStoredNumber == 1;
						}
						else
						{
							return RhsStoredNumber == 0;
						}
					}
				}, Rhs.Get<JsonNumberValueVariants>());
		}
	}
	else if (Lhs.IsType<JsonNumberValueVariants>())
	{
		if (Rhs.IsType<JsonNumberValueVariants>())
		{
			return Lhs.Get<JsonNumberValueVariants>() == Rhs.Get<JsonNumberValueVariants>();
		}
		else // RhsType.IsType<bool>() || RhsType.IsType<FString>()
		{
			// Swapping args to avoid code duplication
			return Rhs == Lhs;
		}
	}
	else // Lhs.IsType<FString>()
	{
		if (Rhs.IsType<FString>())
		{
			return Lhs.Get<FString>().Equals(Rhs.Get<FString>(), ESearchCase::CaseSensitive);
		}
		else if (Rhs.IsType<bool>())
		{
			// Swapping args to avoid code duplication
			return Rhs == Lhs;
		}
		else // RhsType.IsType<JsonNumberValueVariants>()
		{
			return Lhs.Get<FString>() == Rhs.Get<JsonNumberValueVariants>();
		}
	}
}