// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "JsonGlobals.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


DEFINE_LOG_CATEGORY(LogJson);


/**
 * Implements the Json module.
 */
class FJsonModule
	: public IModuleInterface
{
public:

	// IModuleInterface interface

	virtual void StartupModule( ) override { }
	virtual void ShutdownModule( ) override { }

	virtual bool SupportsDynamicReloading( ) override
	{
		return false;
	}
};


IMPLEMENT_MODULE(FJsonModule, Json);
