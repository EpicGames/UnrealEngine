// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonUtils/JsonConversion.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Logging/StructuredLog.h"
#include "Logging/LogMacros.h"


DEFINE_LOG_CATEGORY_STATIC(LogJsonConversion, Display, All);

namespace UE {
namespace Json {

namespace Private {

	static TOptional<FValue> ConvertToRapidJsonValueInternal(const FJsonValue& SrcValue, FDocument::AllocatorType& Allocator)
	{
		switch (SrcValue.Type)
		{
		case EJson::None:
			return {};

		case EJson::Null:
			return {FValue(rapidjson::kNullType)};

		case EJson::String:
			{
				FString SrcString;

				if (ensure(SrcValue.TryGetString(SrcString)))
				{
					FValue Result(rapidjson::kStringType);
					Result.SetString(*SrcString, Allocator);

					return {MoveTemp(Result)};
				}				
			}
			return {};

		case EJson::Number:
			{
				// all values in shared json discard their original types and are stored as doubles, this means that the reader functions have to do type conversions from doubles
				// to mimic the legacy behaviour as long as a legacy API exists.

				double SrcNumber = 0.0;
				if (ensure(SrcValue.TryGetNumber(SrcNumber)))
				{
					FValue Result(rapidjson::kNumberType);
					Result.SetDouble(SrcNumber);

					return {MoveTemp(Result)};
				}
			}
			return {};

		case EJson::Boolean:
			{
				bool SrcBool = false;
				if (ensure(SrcValue.TryGetBool(SrcBool)))
				{
					return {FValue(SrcBool)};
				}				
			}
			return {};

		case EJson::Array:
			{
				const TArray<TSharedPtr<FJsonValue>>* SrcArray = nullptr;

				if (SrcValue.TryGetArray(SrcArray))
				{
					FValue DstArray(rapidjson::kArrayType);

					for (const TSharedPtr<FJsonValue>& SrcArrayValue : *SrcArray)
					{
						TOptional<FValue> DstArrayValue = ConvertToRapidJsonValueInternal(*SrcArrayValue, Allocator);
						if (DstArrayValue.IsSet())
						{
							DstArray.PushBack(MoveTemp(*DstArrayValue), Allocator);
						}
						else
						{
							// should have logged already
							return {};
						}
					}

					return {MoveTemp(DstArray)};
				}				
			}
			return {};

		case EJson::Object:
			{
				const TSharedPtr<FJsonObject>* SrcObject = nullptr;
				if (SrcValue.TryGetObject(SrcObject))
				{
					FValue DstObject(rapidjson::kObjectType);
					
					for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Member : (*SrcObject)->Values)
					{
						TOptional<FValue> DstMember = ConvertToRapidJsonValueInternal(*Member.Value, Allocator);
						if (DstMember.IsSet())
						{
							FValue KeyType(rapidjson::kStringType);
							KeyType.SetString(*Member.Key, Allocator);
							DstObject.AddMember(MoveTemp(KeyType), MoveTemp(*DstMember), Allocator);
						}
						else
						{
							// should have been logged already
							return {};
						}
					}

					return {MoveTemp(DstObject)};
				}
			}
			return {};
		}

		UE_LOGF(LogJsonConversion, Error, "UE::Json::Private::ConvertToRapidJsonValueInternal Unable to convert unknown default Json type %d", int32{EnumToUnderlyingType(SrcValue.Type)});
		return {};
	}
}

TSharedPtr<FJsonValue> ConvertRapidJsonToSharedJsonValue(const FValue& Value)
{
	switch (Value.GetType())
	{
	case rapidjson::kNullType:
		return MakeShared<FJsonValueNull>();

	case rapidjson::kFalseType:
		return MakeShared<FJsonValueBoolean>(false);

	case rapidjson::kTrueType:
		return MakeShared<FJsonValueBoolean>(true);

	case rapidjson::kObjectType:
		{
			TSharedPtr<FJsonObject> DstObject = MakeShared<FJsonObject>();

			FConstObject SrcObject = Value.GetObject();
			for (FValue::ConstMemberIterator It = SrcObject.MemberBegin(); It != SrcObject.MemberEnd(); ++It)
			{
				TSharedPtr<FJsonValue> ChildValue = ConvertRapidJsonToSharedJsonValue(It->value);
				if (ChildValue.IsValid())
				{
					DstObject->SetField(It->name.GetString(), MoveTemp(ChildValue));
				}
			}

			return MakeShared<FJsonValueObject>(MoveTemp(DstObject));
		}

	case rapidjson::kArrayType:
		{
			TArray<TSharedPtr<FJsonValue>> DstArray;

			for (const FValue& SrcValue : Value.GetArray())
			{
				TSharedPtr<FJsonValue> DstValue = ConvertRapidJsonToSharedJsonValue(SrcValue);
				if (DstValue.IsValid())
				{
					DstArray.Add(MoveTemp(DstValue));
				}
			}

			return MakeShared<FJsonValueArray>(MoveTemp(DstArray));
		}

	case rapidjson::kStringType:
		return MakeShared<FJsonValueString>(Value.GetString());

	case rapidjson::kNumberType:
		if (Value.IsDouble())
		{
			return MakeShared<FJsonValueNumber>(Value.GetDouble());
		}
		else if (Value.IsInt64())
		{
			return MakeShared<FJsonValueNumber>(static_cast<double>(Value.GetInt64()));
		}
		else if (Value.IsUint64())
		{
			return MakeShared<FJsonValueNumber>(static_cast<double>(Value.GetUint64()));
		}
	}

	UE_LOGF(LogJsonConversion, Warning, "UE::Json::ConvertRapidJsonToSharedJsonValue Unhandled value type %d", int32(Value.GetType()));
	return {};
}

TSharedPtr<FJsonObject> ConvertRapidJsonToSharedJsonObject(FConstObject Object)
{
	TSharedPtr<FJsonValue> ResultValue = ConvertRapidJsonToSharedJsonValue(Object);
	if (!ResultValue.IsValid() || ResultValue->Type != EJson::Object)
	{
		UE_LOGF(LogJsonConversion, Error, "UE::Json::ConvertRapidJsonToSharedJsonObject unable to convert JSON object");
		return {};
	}

	return ResultValue->AsObject();
}

TOptional<FDocument> ConvertSharedJsonToRapidJsonDocument(const FJsonObject& SrcObject)
{
	FDocument DstDocument;
	DstDocument.SetObject();
	FDocument::AllocatorType& Allocator = DstDocument.GetAllocator();

	for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& SrcField : SrcObject.Values)
	{
		TOptional<FValue> DstMember = Private::ConvertToRapidJsonValueInternal(*SrcField.Value, Allocator);
		if (DstMember.IsSet())
		{
			FValue KeyType(rapidjson::kStringType);
			KeyType.SetString(*SrcField.Key, Allocator);

			DstDocument.AddMember(MoveTemp(KeyType), MoveTemp(*DstMember), Allocator);
		}
	}

	return MoveTemp(DstDocument);
}

}} // namespace UE::Json
