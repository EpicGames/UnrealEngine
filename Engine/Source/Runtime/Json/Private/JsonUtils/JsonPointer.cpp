// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonUtils/JsonPointer.h"

#include "Dom/JsonObject.h"

namespace UE::Json
{
	namespace Private
	{
		// Stored as null-terminated strings in order to work with built-in string functions

		static constexpr TCHAR RootCharacterStr[] = TEXT("#");
		static constexpr TCHAR PathDelimiterCharacterStr[] = TEXT("/");
		static constexpr TCHAR ArrayAppendCharacterStr[] = TEXT("-"); // unused, but kept for reference as official part of spec
	}

	FJsonPointer::FJsonPointer(FStringView InPath)
	{
		ParsePath(FString(InPath));
	}

	FJsonPointer::FJsonPointer(const FJsonPointer& Other)
		: PathParts(Other.PathParts)
	{
	}

	FJsonPointer::FJsonPointer(FJsonPointer&& Other) noexcept
		: PathParts(MoveTemp(Other.PathParts))
	{
	}

	static TSharedPtr<FJsonValue> FindPartInJsonValue(const TSharedPtr<FJsonValue>& Value, const FString& PathPart)
	{
		TSharedPtr<FJsonValue> NextValue = Value;

		TArray<TSharedPtr<FJsonValue>>* ArrayPtr = nullptr;
		if (NextValue->TryGetArray(ArrayPtr))
		{
			if(!PathPart.IsNumeric())
			{
				// part expected to be index
				UE_LOGF(LogJson, Warning, "FJsonPointer::TryGet: JsonValue was an array and expected a numeric part but was a string.");
				return nullptr;
			}
			
			const int32 PartAsIndex = FCString::Atoi(*PathPart);		
			if (!ArrayPtr->IsValidIndex(PartAsIndex))
			{
				// index out of range
				UE_LOGF(LogJson, Warning, "FJsonPointer::TryGet: Array index %d for array is out of range.", PartAsIndex);
				return nullptr;
			}

			NextValue = (*ArrayPtr)[PartAsIndex];
		}
		else
		{
			const TSharedPtr<FJsonObject>* ObjectPtr;
			if(NextValue->TryGetObject(ObjectPtr))
			{
				NextValue = (*ObjectPtr)->GetFieldUntyped(PathPart);
				if (!NextValue.IsValid())
				{
					// part of path didn't exist
					UE_LOGF(LogJson, Warning, "FJsonPointer::TryGet: Field \"%ls\" not found.", *PathPart);
					return {};
				}
			}
		}

		return NextValue;
	}

	bool FJsonPointer::TryGet(const TSharedPtr<FJsonObject>& RootObject, TSharedPtr<FJsonValue>& OutValue) const
	{			
		return TryGet(MakeShared<FJsonValueObject>(RootObject), OutValue);
	}

	bool FJsonPointer::TryGet(const TSharedPtr<FJsonValue>& RootValue, TSharedPtr<FJsonValue>& OutValue) const
	{
		if (!IsValid())
		{
			return false;
		}

		TSharedPtr<FJsonValue> CurrentValue = RootValue;

		for (int32 Idx = 0; Idx < Length(); ++Idx)
		{
			CurrentValue = FindPartInJsonValue(CurrentValue, PathParts[Idx]);
			if (!CurrentValue.IsValid())
			{
				return false;
			}
		}

		OutValue = CurrentValue;
		return true;
	}

	bool FJsonPointer::TryGet(const TArray<TSharedPtr<FJsonValue>>& RootValues, TSharedPtr<FJsonValue>& OutValue) const
	{
		if (!IsValid())
		{
			return false;
		}

		const FString& FirstPart = PathParts[0];
		if(!FirstPart.IsNumeric())
		{
			UE_LOGF(LogJson, Warning, "FJsonPointer::TryGet: RootObject was an array, but JsonPointer path requested a field \"%ls\".", *FirstPart);
			return false;
		}

		const int32 PartAsIndex = FCString::Atoi(*FirstPart);
		if (!RootValues.IsValidIndex(PartAsIndex))
		{
			// index out of range
			UE_LOGF(LogJson, Warning, "FJsonPointer::TryGet: Array index %d for array %ls is out of range.", PartAsIndex, *FirstPart);
			return false;
		}

		return TryGet(RootValues[PartAsIndex], OutValue);
	}

	bool FJsonPointer::ParsePath(const FString& InPath)
	{
		TArray<FString> PathStrings;
		InPath.ParseIntoArray(PathStrings, Private::PathDelimiterCharacterStr, false);

		if(InPath.IsEmpty())
		{
			return false;
		}

		// We don't really care about the document root, remove it if it exists
		if (PathStrings[0] == Private::RootCharacterStr)
		{
			PathStrings.RemoveAt(0);
		}

		for (int32 Idx = 0; Idx < PathStrings.Num(); ++Idx)
		{
			FString PartString = PathStrings[Idx];
			if (PartString.IsEmpty())
			{
				PathParts.Empty();
				UE_LOGF(LogJson, Warning, "Path part was empty in JSON pointer \"%ls\"", *InPath);
				return false;
			}

			PathParts.Add(PartString);
		}

		return true;
	}

	void FJsonPointer::Append(FStringView Part)
	{
		PathParts.Add(FString(Part));
	}

	void FJsonPointer::SetArrayIndex(int32 Index)
	{
		check(PathParts.Num() > 0);

		PathParts.Last() = LexToString(Index);
	}

	FJsonPointer FJsonPointer::GetSubPath(int32 NumParts) const
	{
		// NumParts is or exceeds length, so just return the whole path
		if (NumParts >= Length())
		{
			return *this;
		}

		FJsonPointer SubPath;
		SubPath.PathParts.Reserve(NumParts);

		for (int32 Idx = 0; Idx < NumParts; ++Idx)
		{
			SubPath.PathParts.Add(PathParts[Idx]);
		}

		return SubPath;
	}

	FString FJsonPointer::ToString() const
	{
		return FString(Private::RootCharacterStr) + Private::PathDelimiterCharacterStr + FString::Join(PathParts, Private::PathDelimiterCharacterStr);
	}

	FString FJsonPointer::EscapePart(const FString& Part)
	{
		return Part
			.Replace(TEXT("~"), TEXT("~0"))
			.Replace(TEXT("/"), TEXT("~1"));
	}

	FString FJsonPointer::UnescapePart(const FString& Part)
	{
		// Performed in order according to: https://datatracker.ietf.org/doc/html/rfc6901#section-4
		return Part
			.Replace(TEXT("~1"), TEXT("/"))
			.Replace(TEXT("~0"), TEXT("~"));
	}
}
