// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonUtils/RapidJsonUtils.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

namespace UE::Json
{

const constexpr uint32 DefaultParseFlags = rapidjson::ParseFlag::kParseTrailingCommasFlag;

TOptional<FStringView> GetStringValue(const FValue& Value)
{
	if (Value.IsString())
	{
		return ValueAsStringView(Value);
	}
	return {};
}

TOptional<bool> GetBoolField(FConstObject Object, const TCHAR* FieldName)
{
	FValue::ConstMemberIterator It = Object.FindMember(FieldName);
	if (It == Object.MemberEnd() || !It->value.IsBool())
	{
		return {};
	}

	return It->value.GetBool();
}

TOptional<int32> GetInt32Field(FConstObject Object, const TCHAR* FieldName)
{
	FValue::ConstMemberIterator It = Object.FindMember(FieldName);
	if (It == Object.MemberEnd() || !It->value.IsInt())
	{
		return {};
	}

	return It->value.GetInt();
}

TOptional<uint32> GetUint32Field(FConstObject Object, const TCHAR* FieldName)
{
	FValue::ConstMemberIterator It = Object.FindMember(FieldName);
	if (It == Object.MemberEnd() || !It->value.IsUint())
	{
		return {};
	}

	return It->value.GetUint();
}

TOptional<int64> GetInt64Field(FConstObject Object, const TCHAR* FieldName)
{
	FValue::ConstMemberIterator It = Object.FindMember(FieldName);
	if (It == Object.MemberEnd() || !It->value.IsInt64())
	{
		return {};
	}

	return It->value.GetInt64();
}

TOptional<uint64> GetUint64Field(FConstObject Object, const TCHAR* FieldName)
{
	FValue::ConstMemberIterator It = Object.FindMember(FieldName);
	if (It == Object.MemberEnd() || !It->value.IsUint64())
	{
		return {};
	}

	return It->value.GetUint64();
}


TOptional<double> GetDoubleField(FConstObject Object, const TCHAR* FieldName)
{
	FValue::ConstMemberIterator It = Object.FindMember(FieldName);
	if (It == Object.MemberEnd() || !It->value.IsDouble())
	{
		return {};
	}

	return It->value.GetDouble();
}

TOptional<FStringView> GetStringField(FConstObject Object, const TCHAR* FieldName)
{
	FValue::ConstMemberIterator It = Object.FindMember(FieldName);
	if (It == Object.MemberEnd() || !It->value.IsString())
	{
		return {};
	}

	return ValueAsStringView(It->value);
}

bool HasField(FConstObject Object, const TCHAR* FieldName)
{
	FValue::ConstMemberIterator It = Object.FindMember(FieldName);
	return It != Object.MemberEnd();
}

bool HasNullField(FConstObject Object, const TCHAR* FieldName)
{
	FValue::ConstMemberIterator It = Object.FindMember(FieldName);
	return It != Object.MemberEnd() && It->value.GetType() == rapidjson::kNullType;
}

TOptional<FConstObject> GetObjectField(FConstObject Object, const TCHAR* FieldName)
{
	FValue::ConstMemberIterator It = Object.FindMember(FieldName);
	if (It == Object.MemberEnd() || !It->value.IsObject())
	{
		return {};
	}

	return {It->value.GetObject()};
}

TOptional<FConstObject> GetRootObject(const FDocument& Document)
{
	if (Document.IsObject())
	{
		return {Document.GetObject()};
	}

	return {};
}

TOptional<FConstArray> GetArrayField(FConstObject Object, const TCHAR* FieldName)
{
	FValue::ConstMemberIterator It = Object.FindMember(FieldName);
	if (It == Object.MemberEnd() || !It->value.IsArray())
	{
		return {};
	}

	return {It->value.GetArray()};
}

int32 FindLineNumber(const FStringView JsonText, uint32 Offset)
{
	// 1-index line numbers
	int32 NCount = 1;
	int32 RCount = 1;

	FStringView BeforeError = JsonText.Left(Offset);

	// count number of newlines up util the offset
	for (TCHAR Current : BeforeError)
	{
		if (Current == TCHAR('\n'))
		{
			NCount++;
		}
		else if (Current == TCHAR('\r'))
		{
			RCount++;
		}
	}

	return FMath::Max(NCount, RCount);
}

static int32 FindLineStartOffset(const FStringView JsonText, uint32 LineNum)
{
	if (LineNum <= 1)
	{
		return 0; // start at the beginning of the file
	}

	int32 CurrentNCount = 0;
	int32 CurrentRCount = 0;
	int32 CurrentOffset = 0;

	for (TCHAR Current : JsonText)
	{
		++CurrentOffset;

		// track \r and \n separately, we care about the largest one
		if (Current == TCHAR('\n'))
		{
			CurrentNCount++;
			if (CurrentNCount == LineNum)
			{
				return CurrentOffset;
			}
		}
		else if (Current == TCHAR('\r'))
		{
			CurrentRCount++;
			if (CurrentRCount == LineNum)
			{
				return CurrentOffset;
			}
		}
	}

	return 0;
}

constexpr size_t InternalError_NotNullTerminated = 1;

FString FParseError::CreateMessage(const FStringView JsonText) const
{
	if (ErrorCode == rapidjson::kParseErrorNone)
	{
		// in thie case we use the offset as an extended error code 
		if (Offset == InternalError_NotNullTerminated)
		{
			return TEXT("JsonText must be null terminated");
		}
		else
		{
			return TEXT("Invalid parse error");
		}
	}
	else
	{
		int32 LineNum = FindLineNumber(JsonText, Offset);

		int32 SnippetStart = FindLineStartOffset(JsonText, LineNum-2);
		int32 SnippetEnd = FindLineStartOffset(JsonText, LineNum+1);

		FString Snippet(JsonText.Mid(SnippetStart, SnippetEnd-SnippetStart).TrimStartAndEnd());

		return FString::Printf(
			TEXT("%s, Line %d:\n%s"),
			GetParseError_En(ErrorCode),
			LineNum,
			*Snippet
		);
	}
}

#if PLATFORM_LITTLE_ENDIAN
	using FDocumentEncoding = rapidjson::UTF16LE<TCHAR>;
#else
	using FDocumentEncoding = rapidjson::UTF16BE<TCHAR>;
#endif

TValueOrError<FDocument, FParseError> Parse(const FStringView JsonText)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("UE::Json::Parse");
	FDocument Result;
	Result.Parse<DefaultParseFlags, FDocumentEncoding>(JsonText.GetData(), JsonText.Len());

	if (Result.HasParseError())
	{
		return MakeError(FParseError{
			.ErrorCode = Result.GetParseError(),
			.Offset = Result.GetErrorOffset() / sizeof(TCHAR)
		});
	}

	return MakeValue(MoveTemp(Result));
}

struct FInSituStream
{
	using Ch = TCHAR;

	FInSituStream(TArrayView<TCHAR> InJsonText)
		: JsonText(InJsonText)
	{
	}

	TCHAR Peek() const
	{
		if (ReadCursor < JsonText.Num())
		{
			return JsonText[ReadCursor];
		}
		return '\0';
	}

	TCHAR Take() 
	{
		if (ReadCursor < JsonText.Num())
		{
			return JsonText[ReadCursor++];
		}
		return '\0';
	}

	TCHAR* PutBegin()
	{
		WriteCursor = ReadCursor;
		check (!JsonText.IsEmpty() && WriteCursor < JsonText.Num());
		return &JsonText[WriteCursor];
	}

	void Put(TCHAR C)
	{
		if (WriteCursor < JsonText.Num())
		{
			JsonText[WriteCursor++] = C;
		}
	}

	size_t PutEnd(TCHAR* BeganAt)
	{
		check (!JsonText.IsEmpty() && WriteCursor <= JsonText.Num());
		return &JsonText[WriteCursor] - BeganAt;
	}

	size_t Tell() const
	{
		return ReadCursor;
	}
	
	TArrayView<TCHAR> JsonText;
	int32 ReadCursor = 0;
	int32 WriteCursor = 0;
};

TValueOrError<FDocument, FParseError> ParseInPlace(TArrayView<TCHAR> JsonText)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("UE::Json::ParseInPlace");

	FDocument Result;
	FInSituStream Stream(JsonText);
	Result.ParseStream<DefaultParseFlags | rapidjson::kParseInsituFlag>(Stream);

	if (Result.HasParseError())
	{
		return MakeError(FParseError{
			.ErrorCode = Result.GetParseError(),
			.Offset = Result.GetErrorOffset() / sizeof(TCHAR)
		});
	}

	return MakeValue(MoveTemp(Result));
}

FString WriteCompact(const FDocument& Document)
{
	FStringBuffer StringBuffer;
	FStringWriter StringWriter(StringBuffer);

	Document.Accept(StringWriter);

	return StringBuffer.GetString();
}

FString WritePretty(const FDocument& Document)
{
	FStringBuffer StringBuffer;
	FPrettyStringWriter StringWriter(StringBuffer);
	StringWriter.SetIndent('\t', 1);

	Document.Accept(StringWriter);

	return StringBuffer.GetString();
}

const TCHAR* GetValueTypeName(const FValue& Value)
{
	switch (Value.GetType())
	{
	case rapidjson::kNullType:
		return TEXT("Null");

	case rapidjson::kTrueType:
	case rapidjson::kFalseType:
		return TEXT("Bool");

	case rapidjson::kNumberType:
		return TEXT("Number");

	case rapidjson::kArrayType:
		return TEXT("Array");

	case rapidjson::kObjectType:
		return TEXT("Object");

	case rapidjson::kStringType:
		return TEXT("String");

	default:
		return TEXT("Unknown");
	}
}

} // namespace UE::Json
