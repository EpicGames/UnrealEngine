// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/JsonDataBag.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonSerializerBase.h"

void FJsonDataBag::Serialize(FJsonSerializerBase& Serializer, bool bFlatObject)
{
	if (Serializer.IsLoading())
	{
		// just grab a reference to the underlying JSON object
		JsonObject = Serializer.GetObject();
	}
	else
	{
		if (!bFlatObject)
		{
			Serializer.StartObject();
		}

		if (JsonObject.IsValid())
		{
			for (const auto& It : JsonObject->Values)
			{
				TSharedPtr<FJsonValue> JsonValue = It.Value;
				if (JsonValue.IsValid())
				{
					switch (JsonValue->Type)
					{
						case EJson::Boolean:
						{
							auto Value = JsonValue->AsBool();
							Serializer.Serialize(*It.Key, Value);
							break;
						}
						case EJson::Number:
						{
							auto Value = JsonValue->AsNumber();
							Serializer.Serialize(*It.Key, Value);
							break;
						}
						case EJson::String:
						{
							auto Value = JsonValue->AsString();
							Serializer.Serialize(*It.Key, Value);
							break;
						}
						case EJson::Array:
						{
							// if we have an array, serialize to string and write raw
							FString JsonStr;
							if (NumPrintIndents.IsSet())
							{
								auto Writer = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&JsonStr, *NumPrintIndents);
								FJsonSerializer::Serialize(JsonValue->AsArray(), Writer);
							}
							else
							{
								auto Writer = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&JsonStr);
								FJsonSerializer::Serialize(JsonValue->AsArray(), Writer);
							}
							
							Serializer.WriteIdentifierPrefix(*It.Key);
							Serializer.WriteRawJSONValue(*JsonStr);
							break;
						}
						case EJson::Object:
						{
							// if we have an object, serialize to string and write raw
							FString JsonStr;

							if (NumPrintIndents.IsSet())
							{
								auto Writer = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&JsonStr, *NumPrintIndents);
								FJsonSerializer::Serialize(JsonValue->AsObject().ToSharedRef(), Writer);
							}
							else
							{
								auto Writer = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(&JsonStr);
								FJsonSerializer::Serialize(JsonValue->AsObject().ToSharedRef(), Writer);
							}

							// too bad there's no JsonObject serialization method on FJsonSerializerBase directly :-/
							Serializer.WriteIdentifierPrefix(*It.Key);
							Serializer.WriteRawJSONValue(*JsonStr);
							break;
						}
					}
				}
			}
		}

		if (!bFlatObject)
		{
			Serializer.EndObject();
		}
	}
}

double FJsonDataBag::GetDouble(const FString& Key) const
{
	const auto Json = GetField(Key);
	return Json.IsValid() ? Json->AsNumber() : 0.0;
}

FString FJsonDataBag::GetString(const FString& Key) const
{
	const auto Json = GetField(Key);
	return Json.IsValid() ? Json->AsString() : FString();
}

bool FJsonDataBag::GetBool(const FString& Key) const
{
	const auto Json = GetField(Key);
	return Json.IsValid() ? Json->AsBool() : false;
}

TSharedPtr<const FJsonValue> FJsonDataBag::GetField(const FString& Key) const
{
	if (JsonObject.IsValid())
	{
		return JsonObject->TryGetField(Key);
	}
	return TSharedPtr<const FJsonValue>();
}

void FJsonDataBag::SetFieldJson(const FString& Key, const TSharedPtr<FJsonValue>& Value)
{
	if (!JsonObject.IsValid())
	{
		JsonObject = MakeShared<FJsonObject>();
	}
	JsonObject->SetField(Key, Value);
}

