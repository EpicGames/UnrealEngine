// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/JsonSerializable.h"
#include "Serialization/JsonSerializerReader.h"
#include "Serialization/JsonSerializerWriter.h"

FJsonSerializable::~FJsonSerializable() 
{
}

const FString FJsonSerializable::ToJson(bool bPrettyPrint/*=true*/) const
{
	// Strip away const, because we use a single method that can read/write which requires non-const semantics
	// Otherwise, we'd have to have 2 separate macros for declaring const to json and non-const from json
	return const_cast<FJsonSerializable*>(this)->ToJson(bPrettyPrint);
}

const FUtf8String FJsonSerializable::ToJsonUtf8(bool bPrettyPrint/*=true*/) const
{
	// Strip away const, because we use a single method that can read/write which requires non-const semantics
	// Otherwise, we'd have to have 2 separate macros for declaring const to json and non-const from json
	return const_cast<FJsonSerializable*>(this)->ToJsonUtf8(bPrettyPrint);
}

namespace FJsonSerializablePrivate
{
	template <typename StringType>
	const StringType ToJsonStringWithType(FJsonSerializable& Json, bool bPrettyPrint)
	{
		using CharType = typename StringType::ElementType;
		StringType JsonStr;
		if (bPrettyPrint)
		{
			TSharedRef<TJsonWriter<CharType>> JsonWriter = TJsonWriterFactory<CharType>::Create(&JsonStr);
			FJsonSerializerWriter<CharType> Serializer(JsonWriter);
			Json.Serialize(Serializer, false);
			JsonWriter->Close();
		}
		else
		{
			TSharedRef<TJsonWriter<CharType, TCondensedJsonPrintPolicy<CharType>>> JsonWriter = TJsonWriterFactory<CharType, TCondensedJsonPrintPolicy<CharType>>::Create(&JsonStr);
			FJsonSerializerWriter<CharType, TCondensedJsonPrintPolicy<CharType>> Serializer(JsonWriter);
			Json.Serialize(Serializer, false);
			JsonWriter->Close();
		}
		return JsonStr;
	}
}

const FString FJsonSerializable::ToJson(bool bPrettyPrint/*=true*/)
{
	return FJsonSerializablePrivate::ToJsonStringWithType<FString>(*this, bPrettyPrint);
}

const FUtf8String FJsonSerializable::ToJsonUtf8(bool bPrettyPrint/*=true*/)
{
	return FJsonSerializablePrivate::ToJsonStringWithType<FUtf8String>(*this, bPrettyPrint);
}

bool FJsonSerializable::FromJson(const TCHAR* Json)
{
	return FromJsonStringView(FStringView(Json));
}

bool FJsonSerializable::FromJson(const UTF8CHAR* Json)
{
	return FromJsonStringView(FUtf8StringView(Json));
}

bool FJsonSerializable::FromJson(const FString& Json)
{
	return FromJsonStringView(FStringView(Json));
}

bool FJsonSerializable::FromJson(const FUtf8String& Json)
{
	return FromJsonStringView(FUtf8StringView(Json));
}

bool FJsonSerializable::FromJson(FString&& Json)
{
	TSharedPtr<FJsonObject> JsonObject;
	TSharedRef<TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(MoveTemp(Json));
	if (FJsonSerializer::Deserialize(JsonReader, JsonObject) &&
		JsonObject.IsValid())
	{
		FJsonSerializerReader Serializer(JsonObject);
		Serialize(Serializer, false);
		return true;
	}
	UE_LOGF(LogJson, Warning, "Failed to parse Json from a string: %ls", *JsonReader->GetErrorMessage());
	return false;
}

bool FJsonSerializable::FromJson(FUtf8String&& Json)
{
	return FromJsonStringView(FUtf8StringView(Json));
}

namespace UE::JsonSerializable::Private
{

template<typename CharType>
bool FromJsonStringView(FJsonSerializable* Serializable, TStringView<CharType> JsonStringView)
{
	TSharedPtr<FJsonObject> JsonObject;
	TSharedRef<TJsonReader<CharType> > JsonReader = TJsonReaderFactory<CharType>::CreateFromView(JsonStringView);
	if (FJsonSerializer::Deserialize(JsonReader,JsonObject) &&
		JsonObject.IsValid())
	{
		FJsonSerializerReader Serializer(JsonObject);
		Serializable->Serialize(Serializer, false);
		return true;
	}
	UE_LOGF(LogJson, Warning, "Failed to parse Json from a string: %ls", *JsonReader->GetErrorMessage());
	return false;
}

}

bool FJsonSerializable::FromJsonStringView(FUtf8StringView JsonStringView)
{
	return UE::JsonSerializable::Private::FromJsonStringView(this, JsonStringView);
}

bool FJsonSerializable::FromJsonStringView(FWideStringView JsonStringView)
{
	return UE::JsonSerializable::Private::FromJsonStringView(this, JsonStringView);
}

bool FJsonSerializable::FromJson(TSharedPtr<FJsonObject> JsonObject)
{
	if (JsonObject.IsValid())
	{
		FJsonSerializerReader Serializer(JsonObject);
		Serialize(Serializer, false);
		return true;
	}
	return false;
}
