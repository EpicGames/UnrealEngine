// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/JsonSerializer.h"

bool FJsonSerializerPolicy_JsonObject::GetValueFromState(const FStackState& State, FValue& OutValue)
{
	switch (State.Type)
	{
	case EJson::Object:
		if (!State.Object.IsValid())
		{
			return false;
		}
		OutValue = MakeShared<FJsonValueObject>(State.Object);
		break;
	case EJson::Array:
		OutValue = MakeShared<FJsonValueArray>(State.Array);
		break;
	case EJson::None:
	case EJson::Null:
	case EJson::String:
	case EJson::Number:
	case EJson::Boolean:
	default:
		// FIXME: would be nice to handle non-composite root values but StackState Deserialize just drops them on the floor
		return false;
	}
	return true;
}

bool FJsonSerializerPolicy_JsonObject::GetValueFromState(const FStackState& State, FArrayOfValues& OutArray)
{
	// Returning an empty array is ok.
	if (State.Type != EJson::Array)
	{
		return false;
	}

	OutArray = State.Array;

	return true;
}

bool FJsonSerializerPolicy_JsonObject::GetValueFromState(const FStackState& State, FMapOfValues& OutMap)
{
	// Returning an empty object is ok
	if (!State.Object.IsValid())
	{
		return false;
	}

	OutMap = State.Object;

	return true;
}

void FJsonSerializerPolicy_JsonObject::ResetValue(FValue& OutValue)
{
	OutValue.Reset();
}

void FJsonSerializerPolicy_JsonObject::ReadObjectStart(FStackState& State)
{
	State.Type = EJson::Object;
#if !UE_JSONOBJECT_LEGACY_STRING_KEYS
	State.Object = MakeShared<FJsonObject>(State.StringSet);	// Share string pool with rest of current JSON structure, reduces memory consumption by avoiding string duplication.
#else
	State.Object = MakeShared<FJsonObject>();
#endif
}

void FJsonSerializerPolicy_JsonObject::ReadObjectEnd(FStackState& State, FValue& OutValue)
{
	OutValue = MakeShared<FJsonValueObject>(State.Object);
}

void FJsonSerializerPolicy_JsonObject::ReadArrayStart(FStackState& State)
{
	State.Type = EJson::Array;
}

void FJsonSerializerPolicy_JsonObject::ReadArrayEnd(FStackState& State, FValue& OutValue)
{
	OutValue = MakeShared<FJsonValueArray>(State.Array);
}

void FJsonSerializerPolicy_JsonObject::ReadNull(FValue& OutValue)
{
	OutValue = MakeShared<FJsonValueNull>();
}

void FJsonSerializerPolicy_JsonObject::AddValueToObject(FStackState& State, const FString& Identifier, FValue& NewValue)
{
	State.Object->SetField(Identifier, NewValue);
}

void FJsonSerializerPolicy_JsonObject::AddValueToArray(FStackState& State, FValue& NewValue)
{
	State.Array.Add(NewValue);
}