// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/JsonSerializerReader.h"

#include "Dom/JsonValue.h"

FJsonSerializerReader::FJsonSerializerReader(TSharedPtr<FJsonObject> InJsonObject)
	: JsonObject(InJsonObject)
{
}

FJsonSerializerReader::~FJsonSerializerReader()
{
}

bool FJsonSerializerReader::IsLoading() const 
{ 
	return true; 
}

bool FJsonSerializerReader::IsSaving() const 
{ 
	return false; 
}

TSharedPtr<FJsonObject> FJsonSerializerReader::GetObject() 
{ 
	return JsonObject; 
}

void FJsonSerializerReader::StartObject() 
{
	// Empty on purpose
}

void FJsonSerializerReader::StartObject(FStringView Name) 
{
	// Empty on purpose
}

void FJsonSerializerReader::EndObject() 
{
	// Empty on purpose
}

void FJsonSerializerReader::StartArray() 
{
	// Empty on purpose
}

void FJsonSerializerReader::StartArray(FStringView Name) 
{
	// Empty on purpose
}

void FJsonSerializerReader::EndArray() 
{
	// Empty on purpose
}

void FJsonSerializerReader::Serialize(FStringView Name, int32& Value) 
{
	if (JsonObject->HasTypedField<EJson::Number>(Name))
	{
		JsonObject->TryGetNumberField(Name, Value);
	}
}

void FJsonSerializerReader::Serialize(FStringView Name, uint32& Value) 
{
	if (JsonObject->HasTypedField<EJson::Number>(Name))
	{
		JsonObject->TryGetNumberField(Name, Value);
	}
}

void FJsonSerializerReader::Serialize(FStringView Name, int64& Value) 
{
	if (JsonObject->HasTypedField<EJson::Number>(Name))
	{
		JsonObject->TryGetNumberField(Name, Value);
	}
}

void FJsonSerializerReader::Serialize(FStringView Name, bool& Value) 
{
	if (JsonObject->HasTypedField<EJson::Boolean>(Name))
	{
		Value = JsonObject->GetBoolField(Name);
	}
}

void FJsonSerializerReader::Serialize(FStringView Name, FString& Value) 
{
	if (JsonObject->HasTypedField<EJson::String>(Name))
	{
		Value = JsonObject->GetStringField(Name);
	}
}

void FJsonSerializerReader::Serialize(FStringView Name, FUtf8String& Value)
{
	if (JsonObject->HasTypedField<EJson::String>(Name))
	{
		Value = JsonObject->GetUtf8StringField(Name);
	}
}

void FJsonSerializerReader::Serialize(FStringView Name, FText& Value) 
{
	if (JsonObject->HasTypedField<EJson::String>(Name))
	{
		Value = FText::FromString(JsonObject->GetStringField(Name));
	}
}

void FJsonSerializerReader::Serialize(FStringView Name, float& Value)
{
	if (JsonObject->HasTypedField<EJson::Number>(Name))
	{
		Value = (float)JsonObject->GetNumberField(Name);
	}
}

void FJsonSerializerReader::Serialize(FStringView Name, double& Value) 
{
	if (JsonObject->HasTypedField<EJson::Number>(Name))
	{
		Value = JsonObject->GetNumberField(Name);
	}
}

void FJsonSerializerReader::Serialize(FStringView Name, FDateTime& Value) 
{
	if (JsonObject->HasTypedField<EJson::String>(Name))
	{
		FDateTime::ParseIso8601(*JsonObject->GetStringField(Name), Value);
	}
}

void FJsonSerializerReader::Serialize(FStringView Name, JsonSimpleValueVariant& InVariant)
{
	TSharedPtr<FJsonValue> FieldValue = JsonObject->TryGetField(Name);
	if (FieldValue.IsValid())
	{
		InVariant = UE::Json::ToSimpleJsonVariant(*FieldValue);
	}
}

void FJsonSerializerReader::SerializeArray(FJsonSerializableArray& Array) 
{
	// @todo - higher level serialization is expecting a Json Object
	check(0 && TEXT("Not implemented"));
}

void FJsonSerializerReader::SerializeArray(FStringView Name, FJsonSerializableArray& Array) 
{
	if (JsonObject->HasTypedField<EJson::Array>(Name))
	{
		TArray< TSharedPtr<FJsonValue> > JsonArray = JsonObject->GetArrayField(Name);
		// Iterate all of the keys and their values
		for (TSharedPtr<FJsonValue>& Value : JsonArray)
		{
			Array.Add(Value->AsString());
		}
	}
}

void FJsonSerializerReader::SerializeArray(FStringView Name, FJsonSerializableArrayInt& Array) 
{
	if (JsonObject->HasTypedField<EJson::Array>(Name))
	{
		TArray< TSharedPtr<FJsonValue> > JsonArray = JsonObject->GetArrayField(Name);
		// Iterate all of the keys and their values
		for (TSharedPtr<FJsonValue>& Value : JsonArray)
		{
			Array.Add((int32)Value->AsNumber());
		}
	}
}

void FJsonSerializerReader::SerializeArray(FStringView Name, FJsonSerializableArrayFloat& Array) 
{
	if (JsonObject->HasTypedField<EJson::Array>(Name))
	{
		TArray< TSharedPtr<FJsonValue> > JsonArray = JsonObject->GetArrayField(Name);
		// Iterate all of the keys and their values
		for (TSharedPtr<FJsonValue>& Value : JsonArray)
		{
			Array.Add((float)Value->AsNumber());
		}
	}
}

void FJsonSerializerReader::SerializeMap(FStringView Name, FJsonSerializableKeyValueMap& Map) 
{
	if (JsonObject->HasTypedField<EJson::Object>(Name))
	{
		TSharedPtr<FJsonObject> JsonMap = JsonObject->GetObjectField(Name);
		// Iterate all of the keys and their values
		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Pair : JsonMap->Values)
		{
			Map.Add(FString(Pair.Key), Pair.Value->AsString());
		}
	}
}

void FJsonSerializerReader::SerializeMap(FStringView Name, FJsonSerializableKeyValueMapInt& Map) 
{
	if (JsonObject->HasTypedField<EJson::Object>(Name))
	{
		TSharedPtr<FJsonObject> JsonMap = JsonObject->GetObjectField(Name);
		// Iterate all of the keys and their values
		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Pair : JsonMap->Values)
		{
			const int32 Value = (int32)Pair.Value->AsNumber();
			Map.Add(FString(Pair.Key), Value);
		}
	}
}

void FJsonSerializerReader::SerializeMap(FStringView Name, FJsonSerializableKeyValueMapArrayInt& Map) 
{
	if (JsonObject->HasTypedField<EJson::Object>(Name))
	{
		TSharedPtr<FJsonObject> JsonMap = JsonObject->GetObjectField(Name);
		// Iterate all of the keys and their values
		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Pair : JsonMap->Values)
		{
			FJsonSerializableArrayInt TempArray;
			for (const TSharedPtr<FJsonValue>& ArrayValue : Pair.Value->AsArray())
			{
				int32 IntValue;
				if (ArrayValue.IsValid() && ArrayValue->TryGetNumber(IntValue))
				{
					TempArray.Add(IntValue);
				}
			}
			Map.Add(FString(Pair.Key), TempArray);
		}
	}
}

void FJsonSerializerReader::SerializeMap(FStringView Name, FJsonSerializableKeyValueMapInt64& Map) 
{
	if (JsonObject->HasTypedField<EJson::Object>(Name))
	{
		TSharedPtr<FJsonObject> JsonMap = JsonObject->GetObjectField(Name);
		// Iterate all of the keys and their values
		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Pair : JsonMap->Values)
		{
			const int64 Value = (int64)Pair.Value->AsNumber();
			Map.Add(FString(Pair.Key), Value);
		}
	}
}

void FJsonSerializerReader::SerializeMap(FStringView Name, FJsonSerializableKeyValueMapFloat& Map) 
{
	if (JsonObject->HasTypedField<EJson::Object>(Name))
	{
		TSharedPtr<FJsonObject> JsonMap = JsonObject->GetObjectField(Name);
		// Iterate all of the keys and their values
		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Pair : JsonMap->Values)
		{
			const float Value = (float)Pair.Value->AsNumber();
			Map.Add(FString(Pair.Key), Value);
		}
	}
}

void FJsonSerializerReader::SerializeMap(FStringView Name, FJsonSerializableKeySimpleValueVariantMap& Map)
{
	if (JsonObject->HasTypedField<EJson::Object>(Name))
	{
		TSharedPtr<FJsonObject> JsonMap = JsonObject->GetObjectField(Name);
		// Iterate all of the keys and their values
		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Pair : JsonMap->Values)
		{
			if (Pair.Value.IsValid())
			{
				Map.Add(FString(Pair.Key), UE::Json::ToSimpleJsonVariant(*Pair.Value));
			}
		}
	}
}

void FJsonSerializerReader::SerializeSimpleMap(FJsonSerializableKeyValueMap& Map) 
{
	// Iterate all of the keys and their values, only taking simple types (not array/object), all in string form
	for (auto KeyValueIt = JsonObject->Values.CreateConstIterator(); KeyValueIt; ++KeyValueIt)
	{
		FString Value;
		if (KeyValueIt.Value()->TryGetString(Value))
		{
			Map.Add(FString(KeyValueIt.Key()), MoveTemp(Value));
		}
	}
}

void FJsonSerializerReader::SerializeMapSafe(FStringView Name, FJsonSerializableKeyValueMap& Map) 
{
	if (JsonObject->HasTypedField<EJson::Object>(Name))
	{
		// Iterate all of the keys and their values, only taking simple types (not array/object), all in string form
		TSharedPtr<FJsonObject> JsonMap = JsonObject->GetObjectField(Name);
		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& Pair : JsonMap->Values)
		{
			FString Value;
			if (Pair.Value->TryGetString(Value))
			{
				Map.Add(FString(Pair.Key), MoveTemp(Value));
			}
		}
	}
}

void FJsonSerializerReader::WriteIdentifierPrefix(FStringView Name)
{
	// Should never be called on a reader
	check(false);
}

void FJsonSerializerReader::WriteRawJSONValue(FStringView Value)
{
	// Should never be called on a reader
	check(false);
}

void FJsonSerializerReader::WriteRawJSONValue(FUtf8StringView Value)
{
	// Should never be called on a reader
	check(false);
}
