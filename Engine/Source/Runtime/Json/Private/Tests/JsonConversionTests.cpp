// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreMinimal.h"
#include "JsonUtils/JsonConversion.h"
#include "Misc/AutomationTest.h"
#include "Serialization/JsonSerializer.h"
#include "Tests/TestHarnessAdapter.h"

#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

TEST_CASE_NAMED(FJsonConversionAutomationTest, "System::Engine::FileSystem::JSON::Conversion", "[ApplicationContextMask][SmokeFilter]")
{
	const TCHAR* JsonText = TEXT(R"json(
{
	"int": 1234,
	"round_down": 1.1,
	"round_up" : 1.9,
	"string": "hello world",
	"on": true,
	"off": false,
	"object": {
		"value": 42
	},
	"array": [
		"aaa",
		"bbb",
		"ccc",
		"ddd"
	],	
	"null": null
}

)json");

	// Checks to make a RapidJson -> default Json conversion is identical
	auto CheckEqual = [&](const TSharedPtr<FJsonObject>& JsonObjectA, const UE::Json::FDocument& RapidJsonDocument)
	{
		TOptional<UE::Json::FConstObject> RootObject = UE::Json::GetRootObject(RapidJsonDocument);
		REQUIRE(RootObject.IsSet());

		TSharedPtr<FJsonObject> JsonObjectB = UE::Json::ConvertRapidJsonToSharedJsonObject(*RootObject);
		REQUIRE(JsonObjectB);

		TSharedPtr<FJsonValue> IntValueA = JsonObjectA->TryGetField(TEXT("int"));
		TSharedPtr<FJsonValue> IntValueB = JsonObjectB->TryGetField(TEXT("int"));
		REQUIRE(IntValueA);
		REQUIRE(IntValueB);
		REQUIRE(IntValueA->Type == EJson::Number);
		REQUIRE(IntValueB->Type == EJson::Number);
		REQUIRE(IntValueA->AsNumber() == IntValueB->AsNumber());

		TSharedPtr<FJsonValue> RoundDownValueA = JsonObjectA->TryGetField(TEXT("round_down"));
		TSharedPtr<FJsonValue> RoundDownValueB = JsonObjectB->TryGetField(TEXT("round_down"));
		REQUIRE(RoundDownValueA);
		REQUIRE(RoundDownValueB);
		REQUIRE(RoundDownValueA->Type == EJson::Number);
		REQUIRE(RoundDownValueB->Type == EJson::Number);
		REQUIRE(RoundDownValueA->AsNumber() == RoundDownValueB->AsNumber());
		// default JSON does some rounding when converting, ensure that is kept during the conversion
		int32 RoundDownA = 0;
		int32 RoundDownB = 0;
		REQUIRE(RoundDownValueA->TryGetNumber(RoundDownA));
		REQUIRE(RoundDownValueB->TryGetNumber(RoundDownB));
		REQUIRE(RoundDownA == RoundDownB);

		TSharedPtr<FJsonValue> RoundUpValueA = JsonObjectA->TryGetField(TEXT("round_down"));
		TSharedPtr<FJsonValue> RoundUpValueB = JsonObjectB->TryGetField(TEXT("round_down"));
		REQUIRE(RoundUpValueA);
		REQUIRE(RoundUpValueB);
		REQUIRE(RoundUpValueA->Type == EJson::Number);
		REQUIRE(RoundUpValueB->Type == EJson::Number);
		REQUIRE(RoundUpValueA->AsNumber() == RoundUpValueB->AsNumber());
		// default JSON does some rounding when converting from float to integer, ensure that is kept during the conversion
		int32 RoundUpA = 0;
		int32 RoundUpB = 0;
		REQUIRE(RoundUpValueA->TryGetNumber(RoundUpA));
		REQUIRE(RoundUpValueB->TryGetNumber(RoundUpB));
		REQUIRE(RoundUpA == RoundUpB);

		TSharedPtr<FJsonValue> StringValueA = JsonObjectA->TryGetField(TEXT("string"));
		TSharedPtr<FJsonValue> StringValueB = JsonObjectB->TryGetField(TEXT("string"));
		REQUIRE(StringValueA);
		REQUIRE(StringValueB);
		REQUIRE(StringValueA->Type == EJson::String);
		REQUIRE(StringValueB->Type == EJson::String);
		REQUIRE(StringValueA->AsString() == StringValueB->AsString());

		TSharedPtr<FJsonValue> OnValueA = JsonObjectA->TryGetField(TEXT("on"));
		TSharedPtr<FJsonValue> OnValueB = JsonObjectB->TryGetField(TEXT("on"));
		REQUIRE(OnValueA);
		REQUIRE(OnValueB);
		REQUIRE(OnValueA->Type == EJson::Boolean);
		REQUIRE(OnValueB->Type == EJson::Boolean);
		REQUIRE(OnValueA->AsBool() == OnValueB->AsBool());

		TSharedPtr<FJsonValue> OffValueA = JsonObjectA->TryGetField(TEXT("off"));
		TSharedPtr<FJsonValue> OffValueB = JsonObjectB->TryGetField(TEXT("off"));
		REQUIRE(OffValueA);
		REQUIRE(OffValueB);
		REQUIRE(OffValueA->Type == EJson::Boolean);
		REQUIRE(OffValueB->Type == EJson::Boolean);
		REQUIRE(OffValueA->AsBool() == OffValueB->AsBool());

		TSharedPtr<FJsonValue> ObjectValueA = JsonObjectA->TryGetField(TEXT("object"));
		TSharedPtr<FJsonValue> ObjectValueB = JsonObjectB->TryGetField(TEXT("object"));
		REQUIRE(ObjectValueA);
		REQUIRE(ObjectValueB);
		REQUIRE(ObjectValueA->Type == EJson::Object);
		REQUIRE(ObjectValueB->Type == EJson::Object);

		TSharedPtr<FJsonObject> ObjectA = ObjectValueA->AsObject();
		TSharedPtr<FJsonObject> ObjectB = ObjectValueB->AsObject();
		REQUIRE(ObjectA);
		REQUIRE(ObjectB);

		TSharedPtr<FJsonValue> ObjectFieldA = ObjectA->TryGetField(TEXT("value"));		
		TSharedPtr<FJsonValue> ObjectFieldB = ObjectB->TryGetField(TEXT("value"));
		REQUIRE(ObjectFieldA);
		REQUIRE(ObjectFieldB);
		REQUIRE(ObjectFieldA->Type == EJson::Number);
		REQUIRE(ObjectFieldB->Type == EJson::Number);
		REQUIRE(ObjectFieldA->AsNumber() == ObjectFieldB->AsNumber());

		TSharedPtr<FJsonValue> ArrayValueA = JsonObjectA->TryGetField(TEXT("array"));
		TSharedPtr<FJsonValue> ArrayValueB = JsonObjectB->TryGetField(TEXT("array"));
		REQUIRE(ArrayValueA);
		REQUIRE(ArrayValueB);
		REQUIRE(ArrayValueA->Type == EJson::Array);
		REQUIRE(ArrayValueB->Type == EJson::Array);

		TArray<TSharedPtr<FJsonValue>> ArrayA = ArrayValueA->AsArray();
		TArray<TSharedPtr<FJsonValue>> ArrayB = ArrayValueB->AsArray();

		REQUIRE(!ArrayA.IsEmpty());
		REQUIRE(!ArrayB.IsEmpty());

		REQUIRE(ArrayA.Num() == ArrayB.Num());

		for (int32 Idx = 0; Idx < ArrayA.Num(); ++Idx)
		{
			const TSharedPtr<FJsonValue>& ItemA = ArrayA[Idx];
			const TSharedPtr<FJsonValue>& ItemB = ArrayB[Idx];

			REQUIRE(ItemA);
			REQUIRE(ItemB);
			REQUIRE(ItemA->Type == EJson::String);
			REQUIRE(ItemB->Type == EJson::String);
			REQUIRE(ItemA->AsString() == ItemB->AsString());
		}
	};


	TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(JsonText);
	TSharedPtr< FJsonObject > DefaultLoadedJson;
	REQUIRE(FJsonSerializer::Deserialize(Reader, DefaultLoadedJson));

	// convert to RapidJson
	TOptional<UE::Json::FDocument> ConvertedDocument = UE::Json::ConvertSharedJsonToRapidJsonDocument(*DefaultLoadedJson);
	REQUIRE(ConvertedDocument.IsSet());
	
	CheckEqual(DefaultLoadedJson, *ConvertedDocument);

	TValueOrError<UE::Json::FDocument, UE::Json::FParseError> LoadedDocument = UE::Json::Parse(JsonText);

	FString ErrorMessage = LoadedDocument.HasError() ? LoadedDocument.GetError().CreateMessage(JsonText) : FString();
	REQUIRE_MESSAGE(ErrorMessage, LoadedDocument.HasValue());

	CheckEqual(DefaultLoadedJson, LoadedDocument.GetValue());
}

#endif // WITH_TESTS
