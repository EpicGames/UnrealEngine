// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonUtils/JsonPointer.h"
#include "Misc/AutomationTest.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

#if WITH_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FJsonPointerSpec, TEXT("JsonPointer"), EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)
TSharedPtr<FJsonObject> JsonObject;
END_DEFINE_SPEC(FJsonPointerSpec)

void FJsonPointerSpec::Define()
{
	BeforeEach([this]
	{
		static constexpr const TCHAR* ExampleJson = TEXT(R"_JSON(
		{
		    "someNumber": 42,
			"someObject": {
				"someNestedNumber": 24
			},
			"someArray": [
				5,
				3,
				22,
				2,
				44
			]
		})_JSON");

		const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(ExampleJson);
		return FJsonSerializer::Deserialize(JsonReader, JsonObject);
	});

	Describe(TEXT("Value"), [this]
	{
		It(TEXT("Existing value is found and returned"), [this]
		{
			const UE::Json::FJsonPointer JsonPointer{TEXT("#/someNumber"});
			
			TSharedPtr<FJsonValue> JsonValue;
			const bool bFound = JsonPointer.TryGet(JsonObject, JsonValue);

			constexpr int32 ExpectedValue = 42;
			
			if(TestTrue(TEXT("Value is found"), bFound))
			{
				TestEqual(TEXT("Value equals 42"), StaticCast<int32>(JsonValue->AsNumber()), ExpectedValue);
			}
		});

		It(TEXT("Existing nested value is found and returned"), [this]
		{
			const UE::Json::FJsonPointer JsonPointer{TEXT("#/someObject/someNestedNumber"});
						
			TSharedPtr<FJsonValue> JsonValue;
			const bool bFound = JsonPointer.TryGet(JsonObject, JsonValue);

			constexpr int32 ExpectedValue = 24;
						
			if(TestTrue(TEXT("Value is found"), bFound))
			{
				TestEqual(TEXT("Value equals 42"), StaticCast<int32>(JsonValue->AsNumber()), ExpectedValue);
			}
		});
		
		It(TEXT("Non-existent value logs a warning"), [this]
		{
			const UE::Json::FJsonPointer JsonPointer{TEXT("#/foo/doesnt/exist"});

			AddExpectedError(TEXT("not found"), EAutomationExpectedErrorFlags::Contains);
						
			TSharedPtr<FJsonValue> JsonValue;
			const bool bFound = JsonPointer.TryGet(JsonObject, JsonValue);

			TestFalse(TEXT("Value is not found"), bFound);
		});

		Describe(TEXT("Values with numeric field names are distinguised from array values"), [this]
		{
			BeforeEach([this]
			{
				JsonObject.Reset();
				
				// Overrides what happens in outer BeforeEach
				static constexpr const TCHAR* ExampleJson = TEXT(R"_JSON(
				{
				    "0": "foo",
					"nested": {
						"1": "bar"
					},
					"2": {
						"bla": false
					},
					"someArray": [
					{
						"Foo": "foo",
						"Bar": "bar"
					}]
				})_JSON");

				const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(ExampleJson);
				return FJsonSerializer::Deserialize(JsonReader, JsonObject);
			});

			It(TEXT("Existing value with numeric field name \"0\" is found and returned as a value, rather than an item in an array"), [this]
			{
				const UE::Json::FJsonPointer JsonPointer{TEXT("#/0"});
				
				TSharedPtr<FJsonValue> JsonValue;
				const bool bFound = JsonPointer.TryGet(JsonObject, JsonValue);

				const FString ExpectedValue = "foo";

				if(TestTrue(TEXT("Value is found"), bFound))
				{
					TestEqual(TEXT("Value equals \"foo\""), JsonValue->AsString(), ExpectedValue);
				}
			});

			It(TEXT("Existing nested value with numeric field name \"1\" is found and returned as a value, rather than an item in an array"), [this]
			{
				const UE::Json::FJsonPointer JsonPointer{TEXT("#/nested/1"});
				
				TSharedPtr<FJsonValue> JsonValue;
				const bool bFound = JsonPointer.TryGet(JsonObject, JsonValue);

				const FString ExpectedValue = "bar";

				if(TestTrue(TEXT("Value is found"), bFound))
				{
					TestEqual(TEXT("Value equals \"bar\""), JsonValue->AsString(), ExpectedValue);
				}
			});

			It(TEXT("Existing nested value where the parent has a numeric field name \"2\" is found and returned"), [this]
			{
				const UE::Json::FJsonPointer JsonPointer{TEXT("#/2/bla"});
									
				TSharedPtr<FJsonValue> JsonValue;
				const bool bFound = JsonPointer.TryGet(JsonObject, JsonValue);

				constexpr bool ExpectedValue = false;

				if(TestTrue(TEXT("Value is found"), bFound))
				{
					TestEqual(TEXT("Value equals false"), JsonValue->AsBool(), ExpectedValue);
				}
			});

			It(TEXT("Existing value within an object, within an array is found and returned"), [this]
			{
				const UE::Json::FJsonPointer JsonPointer{TEXT("#/someArray/0/Bar"});
												
				TSharedPtr<FJsonValue> JsonValue;
				const bool bFound = JsonPointer.TryGet(JsonObject, JsonValue);

				const FString ExpectedValue = "bar";

				if(TestTrue(TEXT("Value is found"), bFound))
				{
					TestEqual(TEXT("Value equals false"), JsonValue->AsString(), ExpectedValue);
				}
			});
		});
	});

	Describe(TEXT("Array Value"), [this]
	{
		It(TEXT("Existing value is found and returned at index 2"), [this]
		{
			const UE::Json::FJsonPointer JsonPointer{TEXT("#/someArray/2"});
			
			TSharedPtr<FJsonValue> JsonValue;
			const bool bFound = JsonPointer.TryGet(JsonObject, JsonValue);

			constexpr int32 ExpectedValue = 22;
			
			if(TestTrue(TEXT("Value is found"), bFound))
			{
				TestEqual(TEXT("Value equals 22"), StaticCast<int32>(JsonValue->AsNumber()), ExpectedValue);
			}
		});

		It(TEXT("Out of range index 5 logs a warning"), [this]
		{
			const UE::Json::FJsonPointer JsonPointer{TEXT("#/someArray/5"});

			AddExpectedError(TEXT("out of range"), EAutomationExpectedErrorFlags::Contains);
			
			TSharedPtr<FJsonValue> JsonValue;
			const bool bFound = JsonPointer.TryGet(JsonObject, JsonValue);

			TestFalse(TEXT("Value is not found"), bFound);
		});

		// When the document root is an array
		It(TEXT("Existing value in root is found and returned at index 2"), [this]
		{
			// Overrides what happens in BeforeEach
			static constexpr const TCHAR* ExampleJson = TEXT("[ 5, 9, 7, 2 ]");

			TArray<TSharedPtr<FJsonValue>> JsonObjects;
			const TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(ExampleJson);			
			if(TestTrue(TEXT("Deserialized when root is array"), FJsonSerializer::Deserialize(JsonReader, JsonObjects)))
			{
				const UE::Json::FJsonPointer JsonPointer{TEXT("#/2"});
					
				TSharedPtr<FJsonValue> JsonValue;
				const bool bFound = JsonPointer.TryGet(JsonObjects, JsonValue);

				constexpr int32 ExpectedValue = 7;

				if(TestTrue(TEXT("Value is found"), bFound))
				{
					TestEqual(TEXT("Value equals 7"), StaticCast<int32>(JsonValue->AsNumber()), ExpectedValue);
				}
			}
		});

		It(TEXT("Path part immediately following an array field must be an index"), [this]
		{
			const UE::Json::FJsonPointer JsonPointer{TEXT("#/someArray/Bar")};
			
			AddExpectedError(TEXT("expected a numeric part"));
			
			TSharedPtr<FJsonValue> JsonValue;
			const bool bFound = JsonPointer.TryGet(JsonObject, JsonValue);

			const FString ExpectedValue = "bar";

			TestFalse(TEXT("Value is found"), bFound);
		});
	});

	Describe(TEXT("Syntax"), [this]
	{
		// Tests https://datatracker.ietf.org/doc/html/rfc6901#section-4
		It(TEXT("Escapes ~ and /"), [this]
		{
			static constexpr const TCHAR* PartToTest = TEXT(R"-(Some~part/with~escapable//Characters)-");
			static constexpr const TCHAR* PartToExpect = TEXT(R"-(Some~0part~1with~0escapable~1~1Characters)-");

			const FString EscapedPart = UE::Json::FJsonPointer::EscapePart(PartToTest);

			TestEqual(TEXT("\"~\" and \"/\" are replaced with \"~0\" and \"~1\" respectively"), EscapedPart, PartToExpect);
		});

		// Tests https://datatracker.ietf.org/doc/html/rfc6901#section-4
		It(TEXT("Unescapes ~ and /"), [this]
		{
			static constexpr const TCHAR* PartToTest = TEXT(R"-(Some~0part~1with~0escapable~1~1Characters)-");
			static constexpr const TCHAR* PartToExpect = TEXT(R"-(Some~part/with~escapable//Characters)-");

			const FString UnescapedPart = UE::Json::FJsonPointer::UnescapePart(PartToTest);

			TestEqual(TEXT("\"~0\" and \"~1\" are replaced with \"~\" and \"/\" respectively"), UnescapedPart, PartToExpect);
		});
	});
}

#endif
