// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "CoreMinimal.h"
#include "Containers/AnsiString.h"
#include "Misc/AutomationTest.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializable.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonSerializerMacros.h"
#include "Serialization/JsonTypes.h"
#include "Tests/TestHarnessAdapter.h"

#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

typedef TJsonWriterFactory< TCHAR, TCondensedJsonPrintPolicy<TCHAR> > FCondensedJsonStringWriterFactory;
typedef TJsonWriter< TCHAR, TCondensedJsonPrintPolicy<TCHAR> > FCondensedJsonStringWriter;
typedef TJsonWriterFactory< ANSICHAR, TCondensedJsonPrintPolicy<ANSICHAR> > FCondensedJsonAnsiStringWriterFactory;
typedef TJsonWriter< ANSICHAR, TCondensedJsonPrintPolicy<ANSICHAR> > FCondensedJsonAnsiStringWriter;
typedef TJsonWriterFactory< UTF8CHAR, TCondensedJsonPrintPolicy<UTF8CHAR> > FCondensedJsonUtf8StringWriterFactory;
typedef TJsonWriter< UTF8CHAR, TCondensedJsonPrintPolicy<UTF8CHAR> > FCondensedJsonUtf8StringWriter;

typedef TJsonWriterFactory< TCHAR, TPrettyJsonPrintPolicy<TCHAR> > FPrettyJsonStringWriterFactory;
typedef TJsonWriter< TCHAR, TPrettyJsonPrintPolicy<TCHAR> > FPrettyJsonStringWriter;

TEST_CASE_NAMED(FJsonAutomationTest, "System::Engine::FileSystem::JSON", "[ApplicationContextMask][SmokeFilter]")
{
	// Null Case
	{
		const FString InputString;
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create( InputString );

		TSharedPtr<FJsonObject> Object;
		REQUIRE(FJsonSerializer::Deserialize( Reader, Object ) == false);
		REQUIRE(!Object.IsValid());
	}

	// Empty Object Case
	{
		const FString InputString = "{}";
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create( InputString );

		TSharedPtr<FJsonObject> Object;
		REQUIRE(FJsonSerializer::Deserialize( Reader, Object ));
		REQUIRE(Object.IsValid());

		FString OutputString;
		TSharedRef< FCondensedJsonStringWriter > Writer = FCondensedJsonStringWriterFactory::Create( &OutputString );
		REQUIRE(FJsonSerializer::Serialize( Object.ToSharedRef(), Writer ));
		REQUIRE(InputString == OutputString);
	}

	// Empty Array Case
	{
		const FString InputString = "[]";
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create( InputString );

		TArray< TSharedPtr<FJsonValue> > Array;
		REQUIRE(FJsonSerializer::Deserialize( Reader, Array ));
		REQUIRE(Array.Num() == 0);

		FString OutputString;
		TSharedRef< FCondensedJsonStringWriter > Writer = FCondensedJsonStringWriterFactory::Create( &OutputString );
		REQUIRE(FJsonSerializer::Serialize( Array, Writer ));
		REQUIRE(InputString == OutputString);
	}

	// Empty Array with Empty Identifier Case
	{
		const FString ExpectedString = "[]";
		FString OutputString;
		TSharedRef<FJsonValueArray> EmptyValuesArray = MakeShared<FJsonValueArray>(TArray<TSharedPtr<FJsonValue>>());
		TSharedRef<FCondensedJsonStringWriter> JsonWriter = FCondensedJsonStringWriterFactory::Create( &OutputString );
		REQUIRE(FJsonSerializer::Serialize(EmptyValuesArray, FString(), JsonWriter));
		REQUIRE(ExpectedString == OutputString);
	}

	// Serializing Object Value with Empty Identifier Case
	{
		const FString ExpectedString = "{\"\":\"foo\"}";
		FString OutputString;
		TSharedRef<FJsonValue> FooValue = MakeShared<FJsonValueString>("foo");
		TSharedRef<FCondensedJsonStringWriter> JsonWriter = FCondensedJsonStringWriterFactory::Create(&OutputString);
		JsonWriter->WriteObjectStart();
		REQUIRE(FJsonSerializer::Serialize(FooValue, FString(), JsonWriter, false));
		JsonWriter->WriteObjectEnd();
		JsonWriter->Close();
		REQUIRE(ExpectedString == OutputString);
	}

	// Simple Array Case
	{
		const FString InputString = 
			TEXT(
				"["
					"{"
						"\"Value\":\"Some String\""
					"}"
				"]"
			);

		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create( InputString );

		TArray< TSharedPtr<FJsonValue> > Array;
		bool bSuccessful = FJsonSerializer::Deserialize(Reader, Array);
		REQUIRE(bSuccessful);
		REQUIRE(Array.Num() == 1);
		REQUIRE(Array[0].IsValid());

		TSharedPtr< FJsonObject > Object = Array[0]->AsObject();
		REQUIRE(Object.IsValid());
		REQUIRE(Object->GetStringField( TEXT("Value") ) == TEXT("Some String"));

		FString OutputString;
		TSharedRef< FCondensedJsonStringWriter > Writer = FCondensedJsonStringWriterFactory::Create( &OutputString );
		REQUIRE(FJsonSerializer::Serialize( Array, Writer ));
		REQUIRE(InputString == OutputString);
	}

	// Object Array Case
	{
		const FString InputString =
			TEXT(
				"["
					"{"
						"\"Value\":\"Some String1\""
					"},"
					"{"
						"\"Value\":\"Some String2\""
					"},"
					"{"
						"\"Value\":\"Some String3\""
					"}"
				"]"
			);

		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(InputString);

		TArray< TSharedPtr<FJsonValue> > Array;

		bool bSuccessful = FJsonSerializer::Deserialize(Reader, Array);
		REQUIRE(bSuccessful);
		REQUIRE(Array.Num() == 3);
		REQUIRE(Array[0].IsValid());
		REQUIRE(Array[1].IsValid());
		REQUIRE(Array[2].IsValid());

		TSharedPtr< FJsonObject > Object = Array[0]->AsObject();
		REQUIRE(Object.IsValid());
		REQUIRE(Object->GetStringField(TEXT("Value")) == TEXT("Some String1"));

		Object = Array[1]->AsObject();
		REQUIRE(Object.IsValid());
		REQUIRE(Object->GetStringField(TEXT("Value")) == TEXT("Some String2"));

		Object = Array[2]->AsObject();
		REQUIRE(Object.IsValid());
		REQUIRE(Object->GetStringField(TEXT("Value")) == TEXT("Some String3"));

		FString OutputString;
		TSharedRef< FCondensedJsonStringWriter > Writer = FCondensedJsonStringWriterFactory::Create(&OutputString);
		REQUIRE(FJsonSerializer::Serialize(Array, Writer));
		REQUIRE(InputString == OutputString);
	}

	// FJsonValue operator== Comparison Equality Test
	{
		/* comparing: "Type1_Type2_#" */
		const FString StoredAsType1 =
			TEXT(
				"{"
					"\"bool_string_0\" : false,"
					"\"bool_string_1\" : true,"
					"\"bool_string_2\" : false,"
					"\"bool_string_3\" : true,"

					"\"int_string_0\" : 10,"
					"\"int_string_1\" : 100,"
					
					"\"float_string_0\" : 10.123,"
					"\"float_string_1\" : 100.34,"

					"\"string_string_0\" : \"foo1\","
					"\"string_string_1\" : \"foo2\","
					
					"\"bool_int_0\" : true,"
					"\"bool_int_1\" : false,"

					"\"int_float_0\" : 10,"
					"\"int_float_1\" : 100.00,"
					"\"int_float_2\" : 10,"

					"\"float_bool_0\" : 1.0,"
					"\"float_bool_1\" : 0.0,"
					"\"float_bool_2\" : 1.00000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000001234,"
					"\"float_bool_3\" : 0.00000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000001234"
				"}"
			);

		const FString StoredAsType2 =
			TEXT(
				"{"
					"\"bool_string_0\" : \"false\","
					"\"bool_string_1\" : \"true\","
					"\"bool_string_2\" : \"0\","
					"\"bool_string_3\" : \"1\","

					"\"int_string_0\" : \"10\","
					"\"int_string_1\" : \"100\","

					"\"float_string_0\" : \"10.123\","
					"\"float_string_1\" : \"100.34\","
					
					"\"string_string_0\" : \"foo1\","
					"\"string_string_1\" : \"foo2\","

					"\"bool_int_0\" : 1,"
					"\"bool_int_1\" : 0,"

					"\"int_float_0\" : 10.0,"
					"\"int_float_1\" : 100.00,"
					"\"int_float_2\" : 10.00000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000001234,"

					"\"float_bool_0\" : true,"
					"\"float_bool_1\" : false,"
					"\"float_bool_2\" : true,"
					"\"float_bool_3\" : false"
				"}"
			);

		TSharedRef< TJsonReader<> > TypeReader_1 = TJsonReaderFactory<>::Create(StoredAsType1);
		TSharedRef< TJsonReader<> > TypeReader_2 = TJsonReaderFactory<>::Create(StoredAsType2);

		TSharedPtr<FJsonObject> TypedObject_1;
		REQUIRE(FJsonSerializer::Deserialize(TypeReader_1, TypedObject_1));
		REQUIRE(TypedObject_1.IsValid());

		TSharedPtr<FJsonObject> TypedObject_2;
		REQUIRE(FJsonSerializer::Deserialize(TypeReader_2, TypedObject_2));
		REQUIRE(TypedObject_2.IsValid());

		REQUIRE(TypedObject_1->Values.Num() == TypedObject_2->Values.Num());

		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& KV : TypedObject_1->Values)
		{
			REQUIRE(TypedObject_2->Values.Contains(KV.Key));
			TSharedPtr<FJsonValue> Typed1_FieldValue = KV.Value;
			TSharedPtr<FJsonValue> Typed2_FieldValue = TypedObject_2->Values[KV.Key];

			REQUIRE(*Typed1_FieldValue == *Typed2_FieldValue);
			REQUIRE(*Typed2_FieldValue == *Typed1_FieldValue);
		}
	}

	// FJsonValue operator!= Comparison Inequality Test
	{
		/* comparing: "Type1_Type2_#" */
		const FString StoredAsType1 =
			TEXT(
				"{"
					"\"bool_string_0\" : false,"
					"\"bool_string_1\" : true,"

					"\"int_string_0\" : 10,"
					"\"int_string_1\" : 100,"

					"\"float_string_0\" : 10.123,"
					"\"float_string_1\" : 100.34,"

					// `FJsonValue operator==` uses `FString::operator==` which uses `ESearchCase::IgnoreCase`
					//"\"string_string_0\" : \"foo1\","
					//"\"string_string_1\" : \"foo2\","

					"\"bool_int_0\" : true,"
					"\"bool_int_1\" : false,"

					"\"int_float_0\" : 10,"
					"\"int_float_1\" : 100.00,"
					"\"int_float_2\" : 10,"

					"\"float_bool_0\" : 1.0,"
					"\"float_bool_1\" : 0.0,"
					"\"float_bool_2\" : 2.5,"
					"\"float_bool_3\" : 3.5"
				"}"
			);

		const FString StoredAsType2 =
			TEXT(
				"{"
					"\"bool_string_0\" : \"not_true\","
					"\"bool_string_1\" : \"not_false\","

					"\"int_string_0\" : \"20\","
					"\"int_string_1\" : \"200\","

					"\"float_string_0\" : \"20.123\","
					"\"float_string_1\" : \"200.34\","

					// `FJsonValue operator==` uses `FString::operator==` which uses `ESearchCase::IgnoreCase`
					//"\"string_string_0\" : \"Foo1\","
					//"\"string_string_1\" : \"Foo2\","

					"\"bool_int_0\" : 2,"
					"\"bool_int_1\" : 3,"

					"\"int_float_0\" : 20.0,"
					"\"int_float_1\" : 200.00,"
					"\"int_float_2\" : 10.5,"

					"\"float_bool_0\" : false,"
					"\"float_bool_1\" : true,"
					"\"float_bool_2\" : true,"
					"\"float_bool_3\" : false"
				"}"
			);

		TSharedRef< TJsonReader<> > TypeReader_1 = TJsonReaderFactory<>::Create(StoredAsType1);
		TSharedRef< TJsonReader<> > TypeReader_2 = TJsonReaderFactory<>::Create(StoredAsType2);

		TSharedPtr<FJsonObject> TypedObject_1;
		REQUIRE(FJsonSerializer::Deserialize(TypeReader_1, TypedObject_1));
		REQUIRE(TypedObject_1.IsValid());

		TSharedPtr<FJsonObject> TypedObject_2;
		REQUIRE(FJsonSerializer::Deserialize(TypeReader_2, TypedObject_2));
		REQUIRE(TypedObject_2.IsValid());

		REQUIRE(TypedObject_1->Values.Num() == TypedObject_2->Values.Num());

		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& KV : TypedObject_1->Values)
		{
			REQUIRE(TypedObject_2->Values.Contains(KV.Key));
			TSharedPtr<FJsonValue> Typed1_FieldValue = KV.Value;
			TSharedPtr<FJsonValue> Typed2_FieldValue = TypedObject_2->Values[KV.Key];

			REQUIRE(*Typed1_FieldValue != *Typed2_FieldValue);
			REQUIRE(*Typed2_FieldValue != *Typed1_FieldValue);
		}
	}

	// JsonSimpleValueVariant operator== Comparison Equality Test
	{
		/* comparing: "Type1_Type2_#" */
		const FString StoredAsType1 =
			TEXT(
				"{"
					"\"bool_string_0\" : false,"
					"\"bool_string_1\" : true,"
					"\"bool_string_2\" : false,"
					"\"bool_string_3\" : true,"

					"\"int_string_0\" : 10,"
					"\"int_string_1\" : 100,"

					"\"float_string_0\" : 10.123,"
					"\"float_string_1\" : 100.34,"

					"\"string_string_0\" : \"foo1\","
					"\"string_string_1\" : \"foo2\","

					"\"bool_int_0\" : true,"
					"\"bool_int_1\" : false,"

					"\"int_float_0\" : 10,"
					"\"int_float_1\" : 100.00,"
					"\"int_float_2\" : 10,"

					"\"float_bool_0\" : 1.0,"
					"\"float_bool_1\" : 0.0,"
					"\"float_bool_2\" : 1.00000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000001234,"
					"\"float_bool_3\" : 0.00000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000001234,"
					"\"float_bool_4\" : 0.9999999999999999999999999999999999999999999999999999999999999999999999999876"
				"}"
			);

		const FString StoredAsType2 =
			TEXT(
				"{"
					"\"bool_string_0\" : \"false\","
					"\"bool_string_1\" : \"true\","
					"\"bool_string_2\" : \"0\","
					"\"bool_string_3\" : \"1\","

					"\"int_string_0\" : \"10\","
					"\"int_string_1\" : \"100\","

					"\"float_string_0\" : \"10.123\","
					"\"float_string_1\" : \"100.34\","

					"\"string_string_0\" : \"foo1\","
					"\"string_string_1\" : \"foo2\","

					"\"bool_int_0\" : 1,"
					"\"bool_int_1\" : 0,"

					"\"int_float_0\" : 10.0,"
					"\"int_float_1\" : 100.00,"
					"\"int_float_2\" : 10.00000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000000001234,"

					"\"float_bool_0\" : true,"
					"\"float_bool_1\" : false,"
					"\"float_bool_2\" : true,"
					"\"float_bool_3\" : false,"
					"\"float_bool_4\" : true"
				"}"
			);

		TSharedRef< TJsonReader<> > TypeReader_1 = TJsonReaderFactory<>::Create(StoredAsType1);
		TSharedRef< TJsonReader<> > TypeReader_2 = TJsonReaderFactory<>::Create(StoredAsType2);

		TSharedPtr<FJsonObject> TypedObject_1;
		REQUIRE(FJsonSerializer::Deserialize(TypeReader_1, TypedObject_1));
		REQUIRE(TypedObject_1.IsValid());

		TSharedPtr<FJsonObject> TypedObject_2;
		REQUIRE(FJsonSerializer::Deserialize(TypeReader_2, TypedObject_2));
		REQUIRE(TypedObject_2.IsValid());

		REQUIRE(TypedObject_1->Values.Num() == TypedObject_2->Values.Num());

		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& KV : TypedObject_1->Values)
		{
			REQUIRE(TypedObject_2->Values.Contains(KV.Key));
			TSharedPtr<FJsonValue> Typed1_FieldValue = KV.Value;
			TSharedPtr<FJsonValue> Typed2_FieldValue = TypedObject_2->Values[KV.Key];

			REQUIRE(UE::Json::ToSimpleJsonVariant(*Typed1_FieldValue) == UE::Json::ToSimpleJsonVariant(*Typed2_FieldValue));
			REQUIRE(UE::Json::ToSimpleJsonVariant(*Typed2_FieldValue) == UE::Json::ToSimpleJsonVariant(*Typed1_FieldValue));
		}
	}

	// JsonSimpleValueVariant operator!= Comparison Inequality Test
	{
		/* comparing: "Type1_Type2_#" */
		const FString StoredAsType1 =
			TEXT(
				"{"
					"\"bool_string_0\" : false,"
					"\"bool_string_1\" : true,"

					"\"int_string_0\" : 10,"
					"\"int_string_1\" : 100,"

					"\"float_string_0\" : 10.123,"
					"\"float_string_1\" : 100.34,"

					"\"string_string_0\" : \"foo1\","
					"\"string_string_1\" : \"foo2\","

					"\"bool_int_0\" : true,"
					"\"bool_int_1\" : false,"

					"\"int_float_0\" : 10,"
					"\"int_float_1\" : 100.00,"
					"\"int_float_2\" : 10,"

					"\"float_bool_0\" : 1.0,"
					"\"float_bool_1\" : 0.0,"
					"\"float_bool_2\" : 2.5,"
					"\"float_bool_3\" : 3.5"
				"}"
			);

		const FString StoredAsType2 =
			TEXT(
				"{"
					"\"bool_string_0\" : \"not_true\","
					"\"bool_string_1\" : \"not_false\","

					"\"int_string_0\" : \"20\","
					"\"int_string_1\" : \"200\","

					"\"float_string_0\" : \"20.123\","
					"\"float_string_1\" : \"200.34\","

					"\"string_string_0\" : \"Foo1\","
					"\"string_string_1\" : \"Foo2\","

					"\"bool_int_0\" : 2,"
					"\"bool_int_1\" : 3,"

					"\"int_float_0\" : 20.0,"
					"\"int_float_1\" : 200.00,"
					"\"int_float_2\" : 10.5,"

					"\"float_bool_0\" : false,"
					"\"float_bool_1\" : true,"
					"\"float_bool_2\" : true,"
					"\"float_bool_3\" : false"
				"}"
			);

		TSharedRef< TJsonReader<> > TypeReader_1 = TJsonReaderFactory<>::Create(StoredAsType1);
		TSharedRef< TJsonReader<> > TypeReader_2 = TJsonReaderFactory<>::Create(StoredAsType2);

		TSharedPtr<FJsonObject> TypedObject_1;
		REQUIRE(FJsonSerializer::Deserialize(TypeReader_1, TypedObject_1));
		REQUIRE(TypedObject_1.IsValid());

		TSharedPtr<FJsonObject> TypedObject_2;
		REQUIRE(FJsonSerializer::Deserialize(TypeReader_2, TypedObject_2));
		REQUIRE(TypedObject_2.IsValid());

		REQUIRE(TypedObject_1->Values.Num() == TypedObject_2->Values.Num());

		for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& KV : TypedObject_1->Values)
		{
			REQUIRE(TypedObject_2->Values.Contains(KV.Key));
			TSharedPtr<FJsonValue> Typed1_FieldValue = KV.Value;
			TSharedPtr<FJsonValue> Typed2_FieldValue = TypedObject_2->Values[KV.Key];

			REQUIRE(UE::Json::ToSimpleJsonVariant(*Typed1_FieldValue) != UE::Json::ToSimpleJsonVariant(*Typed2_FieldValue));
			REQUIRE(UE::Json::ToSimpleJsonVariant(*Typed2_FieldValue) != UE::Json::ToSimpleJsonVariant(*Typed1_FieldValue));
		}
	}

	// Number Array Case
	{
		const FString InputString =
			TEXT(
				"["
					"10,"
					"20,"
					"30,"
					"40"
				"]"
			);

		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(InputString);

		TArray< TSharedPtr<FJsonValue> > Array;
		bool bSuccessful = FJsonSerializer::Deserialize(Reader, Array);
		REQUIRE(bSuccessful);
		REQUIRE(Array.Num() == 4);
		REQUIRE(Array[0].IsValid());
		REQUIRE(Array[1].IsValid());
		REQUIRE(Array[2].IsValid());
		REQUIRE(Array[3].IsValid());

		double Number = Array[0]->AsNumber();
		REQUIRE(Number == 10);

		Number = Array[1]->AsNumber();
		REQUIRE(Number == 20);

		Number = Array[2]->AsNumber();
		REQUIRE(Number == 30);

		Number = Array[3]->AsNumber();
		REQUIRE(Number == 40);

		FString OutputString;
		TSharedRef< FCondensedJsonStringWriter > Writer = FCondensedJsonStringWriterFactory::Create(&OutputString);
		REQUIRE(FJsonSerializer::Serialize(Array, Writer));
		REQUIRE(InputString == OutputString);
	}

	// String Array Case
	{
		const FString InputString =
			TEXT(
				"["
					"\"Some String1\","
					"\"Some String2\","
					"\"Some String3\","
					"\"Some String4\""
				"]"
			);

		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(InputString);

		TArray< TSharedPtr<FJsonValue> > Array;
		bool bSuccessful = FJsonSerializer::Deserialize(Reader, Array);
		REQUIRE(bSuccessful);
		REQUIRE(Array.Num() == 4);
		REQUIRE(Array[0].IsValid());
		REQUIRE(Array[1].IsValid());
		REQUIRE(Array[2].IsValid());
		REQUIRE(Array[3].IsValid());

		FString Text = Array[0]->AsString();
		REQUIRE(Text == TEXT("Some String1"));

		Text = Array[1]->AsString();
		REQUIRE(Text == TEXT("Some String2"));

		Text = Array[2]->AsString();
		REQUIRE(Text == TEXT("Some String3"));

		Text = Array[3]->AsString();
		REQUIRE(Text == TEXT("Some String4"));

		FString OutputString;
		TSharedRef< FCondensedJsonStringWriter > Writer = FCondensedJsonStringWriterFactory::Create(&OutputString);
		REQUIRE(FJsonSerializer::Serialize(Array, Writer));
		REQUIRE(InputString == OutputString);
	}

	// Complex Array Case
	{
		const FString InputString =
			TEXT(
				"["
					"\"Some String1\","
					"10,"
					"{"
						"\"\":\"Empty Key\","
						"\"Value\":\"Some String3\""
					"},"
					"["
						"\"Some String4\","
						"\"Some String5\""
					"],"
					"true,"
					"null"
				"]"
			);

		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(InputString);

		TArray< TSharedPtr<FJsonValue> > Array;
		bool bSuccessful = FJsonSerializer::Deserialize(Reader, Array);
		REQUIRE(bSuccessful);
		REQUIRE(Array.Num() == 6);
		REQUIRE(Array[0].IsValid());
		REQUIRE(Array[1].IsValid());
		REQUIRE(Array[2].IsValid());
		REQUIRE(Array[3].IsValid());
		REQUIRE(Array[4].IsValid());
		REQUIRE(Array[5].IsValid());

		FString Text = Array[0]->AsString();
		REQUIRE(Text == TEXT("Some String1"));

		double Number = Array[1]->AsNumber();
		REQUIRE(Number == 10);

		TSharedPtr< FJsonObject > Object = Array[2]->AsObject();
		REQUIRE(Object.IsValid());
		REQUIRE(Object->GetStringField(TEXT("Value")) == TEXT("Some String3"));
		REQUIRE(Object->GetStringField(TEXT("")) == TEXT("Empty Key"));

		const TArray<TSharedPtr< FJsonValue >>& InnerArray = Array[3]->AsArray();
		REQUIRE(InnerArray.Num() == 2);
		REQUIRE(Array[0].IsValid());
		REQUIRE(Array[1].IsValid());

		Text = InnerArray[0]->AsString();
		REQUIRE(Text == TEXT("Some String4"));

		Text = InnerArray[1]->AsString();
		REQUIRE(Text == TEXT("Some String5"));

		bool Boolean = Array[4]->AsBool();
		REQUIRE(Boolean == true);

		REQUIRE(Array[5]->IsNull() == true);

		FString OutputString;
		TSharedRef< FCondensedJsonStringWriter > Writer = FCondensedJsonStringWriterFactory::Create(&OutputString);
		REQUIRE(FJsonSerializer::Serialize(Array, Writer));
		REQUIRE(InputString == OutputString);
	}

	// String Test
	{
		const FString InputString =
			TEXT(
				"{"
					"\"Value\":\"Some String, Escape Chars: \\\\, \\\", \\/, \\b, \\f, \\n, \\r, \\t, \\u002B\""
				"}"
			);
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create( InputString );

		TSharedPtr<FJsonObject> Object;
		bool bSuccessful = FJsonSerializer::Deserialize(Reader, Object);
		REQUIRE(bSuccessful);
		REQUIRE(Object.IsValid());

		const TSharedPtr<FJsonValue> Value = Object->GetFieldUntyped(TEXT("Value"));
		REQUIRE(Value);
		REQUIRE(Value->Type == EJson::String);
		const FString String = Value->AsString();
		REQUIRE(String == TEXT("Some String, Escape Chars: \\, \", /, \b, \f, \n, \r, \t, +"));

		FString OutputString;
		TSharedRef< FCondensedJsonStringWriter > Writer = FCondensedJsonStringWriterFactory::Create( &OutputString );
		REQUIRE(FJsonSerializer::Serialize( Object.ToSharedRef(), Writer ));

		const FString TestOutput =
			TEXT(
				"{"
					"\"Value\":\"Some String, Escape Chars: \\\\, \\\", /, \\b, \\f, \\n, \\r, \\t, +\""
				"}"
			);
		REQUIRE(OutputString == TestOutput);
	}

	// String Test ANSI
	{
		const ANSICHAR* InputString =
			"{" \
				"\"Value\":\"Some String, Escape Chars: \\\\, \\\", \\/, \\b, \\f, \\n, \\r, \\t, \\u002B\\uD83D\\uDE10\""\
			"}";

		TSharedRef< TJsonReader<ANSICHAR> > Reader = TJsonReaderFactory<ANSICHAR>::CreateFromView(InputString);

		TSharedPtr<FJsonObject> Object;
		bool bSuccessful = FJsonSerializer::Deserialize(Reader, Object);
		REQUIRE(bSuccessful);
		REQUIRE(Object.IsValid());

		{
			const TSharedPtr<FJsonValue> Value = Object->GetFieldUntyped(TEXT("Value"));
			REQUIRE((Value && (Value)->Type == EJson::String));
			const FString String = (Value)->AsString();
			REQUIRE(String == TEXT("Some String, Escape Chars: \\, \", /, \b, \f, \n, \r, \t, +😐"));
			const FUtf8String Utf8String = (Value)->AsUtf8String();
			REQUIRE(Utf8String == UTF8TEXT("Some String, Escape Chars: \\, \", /, \b, \f, \n, \r, \t, +😐"));
		}

		FAnsiString AnsiOutputString;
		TSharedRef< FCondensedJsonAnsiStringWriter > AnsiWriter = FCondensedJsonAnsiStringWriterFactory::Create(&AnsiOutputString);
		REQUIRE(FJsonSerializer::Serialize(Object.ToSharedRef(), AnsiWriter));

		const FAnsiString AnsiTestOutput =
			"{" \
				"\"Value\":\"Some String, Escape Chars: \\\\, \\\", /, \\b, \\f, \\n, \\r, \\t, +\\ud83d\\ude10\""\
			"}";
		REQUIRE(AnsiOutputString == AnsiTestOutput);
	}

	// String Test UTF8
	{
		// UTF8TEXT will prepend the first u8 literal specifier
		// UTF8TEXT does a cast, so we can't add it each line and still get the literals to concatenate
		const UTF8CHAR* InputString = UTF8TEXT(
			"{"
				u8"\"Value\":\"Some String, Escape Chars: \\\\, \\\", \\/, \\b, \\f, \\n, \\r, \\t, \\u002B\\uD83D\\uDE10\","
				u8"\"Value1\":\"Greek String, Σὲ γνωρίζω ἀπὸ τὴν κόψη\","
				u8"\"Value2\":\"Thai String, สิบสองกษัตริย์ก่อนหน้าแลถัดไป\","
				u8"\"Value3\":\"Hello world, Καλημέρα κόσμε, コンニチハ\""
			u8"}"
		);

		TSharedRef< TJsonReader<UTF8CHAR> > Reader = TJsonReaderFactory<UTF8CHAR>::CreateFromView( InputString );

		TSharedPtr<FJsonObject> Object;
		bool bSuccessful = FJsonSerializer::Deserialize(Reader, Object);
		REQUIRE(bSuccessful);
		REQUIRE(Object.IsValid());

		{
			const TSharedPtr<FJsonValue> Value = Object->GetFieldUntyped(TEXT("Value"));
			REQUIRE((Value && (Value)->Type == EJson::String));
			const FString String = (Value)->AsString();
			REQUIRE(String == TEXT("Some String, Escape Chars: \\, \", /, \b, \f, \n, \r, \t, +😐"));
			const FUtf8String Utf8String = (Value)->AsUtf8String();
			REQUIRE(Utf8String == UTF8TEXT("Some String, Escape Chars: \\, \", /, \b, \f, \n, \r, \t, +😐"));
		}
		{
			const TSharedPtr<FJsonValue> Value = Object->GetFieldUntyped(TEXT("Value1"));
			REQUIRE((Value && (Value)->Type == EJson::String));
			const FString String = (Value)->AsString();
			REQUIRE(String == TEXT("Greek String, Σὲ γνωρίζω ἀπὸ τὴν κόψη"));
			const FUtf8String Utf8String = (Value)->AsUtf8String();
			REQUIRE(Utf8String == UTF8TEXT("Greek String, Σὲ γνωρίζω ἀπὸ τὴν κόψη"));
		}
		{
			const TSharedPtr<FJsonValue> Value = Object->GetFieldUntyped(TEXT("Value2"));
			REQUIRE((Value && (Value)->Type == EJson::String));
			const FString String = (Value)->AsString();
			REQUIRE(String == TEXT("Thai String, สิบสองกษัตริย์ก่อนหน้าแลถัดไป"));
			const FUtf8String Utf8String = (Value)->AsUtf8String();
			REQUIRE(Utf8String == UTF8TEXT("Thai String, สิบสองกษัตริย์ก่อนหน้าแลถัดไป"));
		}
		{
			const TSharedPtr<FJsonValue> Value = Object->GetFieldUntyped(TEXT("Value3"));
			REQUIRE((Value && (Value)->Type == EJson::String));
			const FString String = (Value)->AsString();
			REQUIRE(String == TEXT("Hello world, Καλημέρα κόσμε, コンニチハ"));
			const FUtf8String Utf8String = (Value)->AsUtf8String();
			REQUIRE(Utf8String == UTF8TEXT("Hello world, Καλημέρα κόσμε, コンニチハ"));
		}

		FString OutputString;
		TSharedRef< FCondensedJsonStringWriter > Writer = FCondensedJsonStringWriterFactory::Create( &OutputString );
		REQUIRE(FJsonSerializer::Serialize( Object.ToSharedRef(), Writer ));

		// Note: The literal prefix for the string (u8, L), must be present for every concatenated string, not just the first one
		const FString TestOutput =
			TEXT("{")
				TEXT("\"Value\":\"Some String, Escape Chars: \\\\, \\\", /, \\b, \\f, \\n, \\r, \\t, +😐\",")
				TEXT("\"Value1\":\"Greek String, Σὲ γνωρίζω ἀπὸ τὴν κόψη\",")
				TEXT("\"Value2\":\"Thai String, สิบสองกษัตริย์ก่อนหน้าแลถัดไป\",")
				TEXT("\"Value3\":\"Hello world, Καλημέρα κόσμε, コンニチハ\"")
			TEXT("}");
		REQUIRE(OutputString == TestOutput);

		FUtf8String Utf8OutputString;
		TSharedRef< FCondensedJsonUtf8StringWriter > Utf8Writer = FCondensedJsonUtf8StringWriterFactory::Create(&Utf8OutputString);
		REQUIRE(FJsonSerializer::Serialize(Object.ToSharedRef(), Utf8Writer));

		// Note: The literal prefix for the string (u8, L), must be present for every concatenated string, not just the first one
		const FUtf8String Utf8TestOutput =
			UTF8TEXT("{"\
						"\"Value\":\"Some String, Escape Chars: \\\\, \\\", /, \\b, \\f, \\n, \\r, \\t, +😐\","\
						"\"Value1\":\"Greek String, Σὲ γνωρίζω ἀπὸ τὴν κόψη\","\
						"\"Value2\":\"Thai String, สิบสองกษัตริย์ก่อนหน้าแลถัดไป\","\
						"\"Value3\":\"Hello world, Καλημέρα κόσμε, コンニチハ\""\
					"}");
		REQUIRE(Utf8OutputString == Utf8TestOutput);
	}

	// Number Test
	{
		const FString InputString =
			TEXT(
				"{"
					"\"Value1\":2.544e+15,"
					"\"Value2\":-0.544E-2,"
					"\"Value3\":251e3,"
					"\"Value4\":-0.0,"
					"\"Value5\":843"
				"}"
			);
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create( InputString );

		TSharedPtr<FJsonObject> Object;
		bool bSuccessful = FJsonSerializer::Deserialize(Reader, Object);
		REQUIRE(bSuccessful);
		REQUIRE(Object.IsValid());

		double TestValues[] = {2.544e+15, -0.544e-2, 251e3, -0.0, 843};
		for (int32 i = 0; i < 5; ++i)
		{
			const TSharedPtr<FJsonValue> Value = Object->GetFieldUntyped(FString::Printf(TEXT("Value%i"), i + 1));
			REQUIRE((Value && (Value)->Type == EJson::Number));
			const double Number = (Value)->AsNumber();
			REQUIRE(Number == TestValues[i]);
		}

		FString OutputString;
		TSharedRef< FCondensedJsonStringWriter > Writer = FCondensedJsonStringWriterFactory::Create( &OutputString );
		REQUIRE(FJsonSerializer::Serialize( Object.ToSharedRef(), Writer ));

		// %g isn't standardized, so we use the same %g format that is used inside PrintJson instead of hardcoding the values here
		const FString TestOutput = FString::Printf(
			TEXT(
				"{"
					"\"Value1\":%.17g,"
					"\"Value2\":%.17g,"
					"\"Value3\":%.17g,"
					"\"Value4\":%.17g,"
					"\"Value5\":%.17g"
				"}"
			),
			TestValues[0], TestValues[1], TestValues[2], TestValues[3], TestValues[4]);
		REQUIRE(OutputString == TestOutput);
	}

	// Test Nan
	auto TestNaN = [&](FJsonSerializer::EFlags Flags) {
		const FString TestNanInd = FString::Printf(TEXT("%.17g"), std::numeric_limits<double>::quiet_NaN());
		CHECK(TestNanInd == TEXT("nan")); // Make sure code will not run on standard library impl which outputs nan(ind)

		const FString InputString =
			TEXT(
				"{"
					"\"Value0\":nan,"
					"\"Value1\":NaN,"
					"\"Value2\":-nan,"
					"\"Value3\":-nan(ind)"
				"}"
			);
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create( InputString );

		TSharedPtr<FJsonObject> Object;
		bool bSuccessful = FJsonSerializer::Deserialize(Reader, Object, Flags);
		REQUIRE(bSuccessful);
		REQUIRE(Object.IsValid());

		const TSharedPtr<FJsonValue> Value0 = Object->GetFieldUntyped(FString::Printf(TEXT("Value%i"), 0));
		const TSharedPtr<FJsonValue> Value1 = Object->GetFieldUntyped(FString::Printf(TEXT("Value%i"), 1));
		const TSharedPtr<FJsonValue> Value2 = Object->GetFieldUntyped(FString::Printf(TEXT("Value%i"), 2));
		const TSharedPtr<FJsonValue> Value3 = Object->GetFieldUntyped(FString::Printf(TEXT("Value%i"), 3));
		const double Number0 = Value0->AsNumber();
		const double Number1 = Value1->AsNumber();
		const double Number2 = Value2->AsNumber();
		const double Number3 = Value3->AsNumber();
		REQUIRE(FMath::IsNaN(Number0));
		REQUIRE(FMath::IsNaN(Number1));
		REQUIRE(FMath::IsNaN(Number2));
		REQUIRE(FMath::IsNaN(Number3));

		FString OutputString;
		TSharedRef< FCondensedJsonStringWriter > Writer = FCondensedJsonStringWriterFactory::Create( &OutputString );
		REQUIRE(FJsonSerializer::Serialize( Object.ToSharedRef(), Writer ));

		FString TestOutput;
		if (Flags == FJsonSerializer::EFlags::StoreNumbersAsStrings)
		{
			TestOutput =
				TEXT(
					"{"
						"\"Value0\":nan,"
						"\"Value1\":NaN,"
						"\"Value2\":-nan,"
						"\"Value3\":-nan(ind)"
					"}"
				);
		}
		else
		{
			// %g isn't standardized, so we use the same %g format that is used inside PrintJson instead of hardcoding the values here
			TestOutput = FString::Printf(
				TEXT(
					"{"
						"\"Value0\":%.17g,"
						"\"Value1\":%.17g,"
						"\"Value2\":%.17g,"
						"\"Value3\":%.17g"
					"}"
				),
				std::numeric_limits<double>::quiet_NaN(), std::numeric_limits<double>::quiet_NaN(), -std::numeric_limits<double>::quiet_NaN(), -std::numeric_limits<double>::quiet_NaN());
		}
		REQUIRE(OutputString == TestOutput);
	};
	TestNaN(FJsonSerializer::EFlags::None);
	TestNaN(FJsonSerializer::EFlags::StoreNumbersAsStrings);

	// Test Inf
	auto TestInf = [&](FJsonSerializer::EFlags Flags) {
		const FString TestNanInd = FString::Printf(TEXT("%.17g"), std::numeric_limits<double>::infinity());
		CHECK(TestNanInd == TEXT("inf"));

		const FString InputString =
			TEXT(
				"{"
					"\"Value0\":inf,"
					"\"Value1\":Inf,"
					"\"Value2\":-inf,"
					"\"Value3\":-Inf"
				"}"
			);
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create(InputString);

		TSharedPtr<FJsonObject> Object;
		bool bSuccessful = FJsonSerializer::Deserialize(Reader, Object, Flags);
		REQUIRE(bSuccessful);
		REQUIRE(Object.IsValid());

		const TSharedPtr<FJsonValue> Value0 = Object->GetFieldUntyped(FString::Printf(TEXT("Value%i"), 0));
		const TSharedPtr<FJsonValue> Value1 = Object->GetFieldUntyped(FString::Printf(TEXT("Value%i"), 1));
		const TSharedPtr<FJsonValue> Value2 = Object->GetFieldUntyped(FString::Printf(TEXT("Value%i"), 2));
		const TSharedPtr<FJsonValue> Value3 = Object->GetFieldUntyped(FString::Printf(TEXT("Value%i"), 3));
		const double Number0 = Value0->AsNumber();
		const double Number1 = Value1->AsNumber();
		const double Number2 = Value2->AsNumber();
		const double Number3 = Value3->AsNumber();
		REQUIRE(Number0 == std::numeric_limits<double>::infinity());
		REQUIRE(Number1 == std::numeric_limits<double>::infinity());
		REQUIRE(Number2 == -std::numeric_limits<double>::infinity());
		REQUIRE(Number3 == -std::numeric_limits<double>::infinity());

		FString OutputString;
		TSharedRef< FCondensedJsonStringWriter > Writer = FCondensedJsonStringWriterFactory::Create(&OutputString);
		REQUIRE(FJsonSerializer::Serialize(Object.ToSharedRef(), Writer));

		FString TestOutput;
		if (Flags == FJsonSerializer::EFlags::StoreNumbersAsStrings)
		{
			TestOutput =
				TEXT(
					"{"
						"\"Value0\":inf,"
						"\"Value1\":Inf,"
						"\"Value2\":-inf,"
						"\"Value3\":-Inf"
					"}"
				);
		}
		else
		{
			// %g isn't standardized, so we use the same %g format that is used inside PrintJson instead of hardcoding the values here
			TestOutput = FString::Printf(
				TEXT(
					"{"
						"\"Value0\":%.17g,"
						"\"Value1\":%.17g,"
						"\"Value2\":%.17g,"
						"\"Value3\":%.17g"
					"}"
				),
				std::numeric_limits<double>::infinity(), std::numeric_limits<double>::infinity(), -std::numeric_limits<double>::infinity(), -std::numeric_limits<double>::infinity());
		}
		REQUIRE(OutputString == TestOutput);
	};
	TestInf(FJsonSerializer::EFlags::None);
	TestInf(FJsonSerializer::EFlags::StoreNumbersAsStrings);

	// Boolean/Null Test
	{
		const FString InputString =
			TEXT(
				"{"
					"\"Value1\":true,"
					"\"Value2\":true,"
					"\"Value3\":faLsE,"
					"\"Value4\":null,"
					"\"Value5\":NULL"
				"}"
			);
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create( InputString );

		TSharedPtr<FJsonObject> Object;
		bool bSuccessful = FJsonSerializer::Deserialize(Reader, Object);
		REQUIRE(bSuccessful);
		REQUIRE(Object.IsValid());

		bool TestValues[] = {true, true, false};
		for (int32 i = 0; i < 5; ++i)
		{
			const TSharedPtr<FJsonValue> Value = Object->GetFieldUntyped(FString::Printf(TEXT("Value%i"), i + 1));
			REQUIRE(Value);
			if (i < 3)
			{
				REQUIRE(Value->Type == EJson::Boolean);
				const bool Bool = Value->AsBool();
				REQUIRE(Bool == TestValues[i]);
			}
			else
			{
				REQUIRE(Value->Type == EJson::Null);
				REQUIRE(Value->IsNull());
			}
		}

		FString OutputString;
		TSharedRef< FCondensedJsonStringWriter > Writer = FCondensedJsonStringWriterFactory::Create( &OutputString );
		REQUIRE(FJsonSerializer::Serialize( Object.ToSharedRef(), Writer ));

		const FString TestOutput =
			TEXT(
				"{"
					"\"Value1\":true,"
					"\"Value2\":true,"
					"\"Value3\":false,"
					"\"Value4\":null,"
					"\"Value5\":null"
				"}"
			);
		REQUIRE(OutputString == TestOutput);
	}

	// Object Test && extra whitespace test
	{
		const FString InputStringWithExtraWhitespace =
			TEXT(
				"		\n\r\n	   {"
					"\"Object\":"
					"{"
						"\"NestedValue\":null,"
						"\"NestedObject\":{}"
					"},"
					"\"Value\":true"
				"}		\n\r\n	   "
			);

		const FString InputString =
			TEXT(
				"{"
					"\"Object\":"
					"{"
						"\"NestedValue\":null,"
						"\"NestedObject\":{}"
					"},"
					"\"Value\":true"
				"}"
			);

		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create( InputStringWithExtraWhitespace );

		TSharedPtr<FJsonObject> Object;
		bool bSuccessful = FJsonSerializer::Deserialize(Reader, Object);
		REQUIRE(bSuccessful);
		REQUIRE(Object.IsValid());

		const TSharedPtr<FJsonValue> InnerValueFail = Object->GetFieldUntyped(TEXT("InnerValue"));
		REQUIRE(!InnerValueFail);

		const TSharedPtr<FJsonValue> ObjectValue = Object->GetFieldUntyped(TEXT("Object"));
		REQUIRE((ObjectValue && ObjectValue->Type == EJson::Object));
		const TSharedPtr<FJsonObject> InnerObject = ObjectValue->AsObject();
		REQUIRE(InnerObject.IsValid());

		{
			const TSharedPtr<FJsonValue> NestedValueValue = InnerObject->GetFieldUntyped(TEXT("NestedValue"));
			REQUIRE((NestedValueValue && NestedValueValue->Type == EJson::Null));
			REQUIRE(NestedValueValue->IsNull());

			const TSharedPtr<FJsonValue> NestedObjectValue = InnerObject->GetFieldUntyped(TEXT("NestedObject"));
			REQUIRE((NestedObjectValue && NestedObjectValue->Type == EJson::Object));
			const TSharedPtr<FJsonObject> InnerInnerObject = NestedObjectValue->AsObject();
			REQUIRE(InnerInnerObject.IsValid());

			{
				const TSharedPtr<FJsonValue> NestedValueValueFail = InnerInnerObject->GetFieldUntyped(TEXT("NestedValue"));
				REQUIRE(!NestedValueValueFail);
			}
		}

		const TSharedPtr<FJsonValue> ValueValue = Object->GetFieldUntyped(TEXT("Value"));
		REQUIRE((ValueValue && ValueValue->Type == EJson::Boolean));
		const bool Bool = ValueValue->AsBool();
		REQUIRE(Bool);

		FString OutputString;
		TSharedRef< FCondensedJsonStringWriter > Writer = FCondensedJsonStringWriterFactory::Create( &OutputString );
		REQUIRE(FJsonSerializer::Serialize( Object.ToSharedRef(), Writer ));
		REQUIRE(OutputString == InputString);
	}

	// Array Test
	{
		const FString InputString =
			TEXT(
				"{"
					"\"Array\":"
					"["
						"[],"
						"\"Some String\","
						"\"Another String\","
						"null,"
						"true,"
						"false,"
						"45,"
						"{}"
					"]"
				"}"
			);
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create( InputString );

		TSharedPtr<FJsonObject> Object;
		bool bSuccessful = FJsonSerializer::Deserialize(Reader, Object);
		REQUIRE(bSuccessful);
		REQUIRE(Object.IsValid());

		const TSharedPtr<FJsonValue> InnerValueFail = Object->GetFieldUntyped(TEXT("InnerValue"));
		REQUIRE(!InnerValueFail);

		const TSharedPtr<FJsonValue> ArrayValue = Object->GetFieldUntyped(TEXT("Array"));
		REQUIRE((ArrayValue && ArrayValue->Type == EJson::Array));
		const TArray< TSharedPtr<FJsonValue> > Array = ArrayValue->AsArray();
		REQUIRE(Array.Num() == 8);

		EJson ValueTypes[] = {EJson::Array, EJson::String, EJson::String, EJson::Null,
			EJson::Boolean, EJson::Boolean, EJson::Number, EJson::Object};
		for (int32 i = 0; i < Array.Num(); ++i)
		{
			const TSharedPtr<FJsonValue>& Value = Array[i];
			REQUIRE(Value.IsValid());
			REQUIRE(Value->Type == ValueTypes[i]);
		}

		const TArray< TSharedPtr<FJsonValue> >& InnerArray = Array[0]->AsArray();
		REQUIRE(InnerArray.Num() == 0);
		REQUIRE(Array[1]->AsString() == TEXT("Some String"));
		REQUIRE(Array[2]->AsString() == TEXT("Another String"));
		REQUIRE(Array[3]->IsNull());
		REQUIRE(Array[4]->AsBool());
		REQUIRE(!Array[5]->AsBool());
		REQUIRE(FMath::Abs(Array[6]->AsNumber() - 45.f) < UE_KINDA_SMALL_NUMBER);
		const TSharedPtr<FJsonObject> InnerObject = Array[7]->AsObject();
		REQUIRE(InnerObject.IsValid());

		FString OutputString;
		TSharedRef< FCondensedJsonStringWriter > Writer = FCondensedJsonStringWriterFactory::Create( &OutputString );
		REQUIRE(FJsonSerializer::Serialize( Object.ToSharedRef(), Writer ));
		REQUIRE(OutputString == InputString);
	}

	// Pretty Print Test
	{
		const FString InputString =
			TEXT(
				"{"											LINE_TERMINATOR_ANSI
					"	\"Data1\": \"value\","				LINE_TERMINATOR_ANSI
					"	\"Data2\": \"value\","				LINE_TERMINATOR_ANSI
					"	\"Array\": ["						LINE_TERMINATOR_ANSI
					"		{"								LINE_TERMINATOR_ANSI
					"			\"InnerData1\": \"value\""	LINE_TERMINATOR_ANSI
					"		},"								LINE_TERMINATOR_ANSI
					"		[],"							LINE_TERMINATOR_ANSI
					"		["								LINE_TERMINATOR_ANSI
					"			1,"							LINE_TERMINATOR_ANSI
					"			2,"							LINE_TERMINATOR_ANSI
					"			3,"							LINE_TERMINATOR_ANSI
					"			4"							LINE_TERMINATOR_ANSI		
					"		],"								LINE_TERMINATOR_ANSI
					"		{"								LINE_TERMINATOR_ANSI
					"		},"								LINE_TERMINATOR_ANSI
					"		\"value\","						LINE_TERMINATOR_ANSI
					"		\"value\""						LINE_TERMINATOR_ANSI
					"	],"									LINE_TERMINATOR_ANSI
					"	\"Object\":"						LINE_TERMINATOR_ANSI
					"	{"									LINE_TERMINATOR_ANSI
					"	}"									LINE_TERMINATOR_ANSI
				"}"
			);
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create( InputString );

		TSharedPtr<FJsonObject> Object;
		REQUIRE(FJsonSerializer::Deserialize( Reader, Object ));
		REQUIRE(Object.IsValid());

		FString OutputString;
		TSharedRef< FPrettyJsonStringWriter > Writer = FPrettyJsonStringWriterFactory::Create( &OutputString );
		REQUIRE(FJsonSerializer::Serialize( Object.ToSharedRef(), Writer ));
		REQUIRE(OutputString == InputString);
	}

	// Line and Character # test
	{
		const FString InputString =
			TEXT(
				"{"											LINE_TERMINATOR_ANSI
					"	\"Data1\": \"value\","				LINE_TERMINATOR_ANSI
					"	\"Array\":"							LINE_TERMINATOR_ANSI
					"	["									LINE_TERMINATOR_ANSI
					"		12345,"							LINE_TERMINATOR_ANSI
					"		True"							LINE_TERMINATOR_ANSI
					"	],"									LINE_TERMINATOR_ANSI
					"	\"Object\":"						LINE_TERMINATOR_ANSI
					"	{"									LINE_TERMINATOR_ANSI
					"	}"									LINE_TERMINATOR_ANSI
				"}"
			);
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create( InputString );

		EJsonNotation Notation = EJsonNotation::Null;
		REQUIRE(( Reader->ReadNext( Notation ) && Notation == EJsonNotation::ObjectStart ));
		REQUIRE(( Reader->GetLineNumber() == 1 && Reader->GetCharacterNumber() == 1 ));

		REQUIRE(( Reader->ReadNext( Notation ) && Notation == EJsonNotation::String ));
		REQUIRE(( Reader->GetLineNumber() == 2 && Reader->GetCharacterNumber() == 17 ));

		REQUIRE(( Reader->ReadNext( Notation ) && Notation == EJsonNotation::ArrayStart ));
		REQUIRE(( Reader->GetLineNumber() == 4 && Reader->GetCharacterNumber() == 2 ));

		REQUIRE(( Reader->ReadNext( Notation ) && Notation == EJsonNotation::Number ));
		REQUIRE(( Reader->GetLineNumber() == 5 && Reader->GetCharacterNumber() == 7 ));

		REQUIRE(( Reader->ReadNext( Notation ) && Notation == EJsonNotation::Boolean ));
		REQUIRE(( Reader->GetLineNumber() == 6 && Reader->GetCharacterNumber() == 6 ));
	}

	// Failure Cases
	TArray<FString> FailureInputs;

	// Unclosed Object
	FailureInputs.Add(
		TEXT("{"));

	// Values in Object without identifiers
	FailureInputs.Add(
		TEXT(
			"{"
				"\"Value1\","
				"\"Value2\","
				"43"
			"}"
		)
	);

	// Unexpected End Of Input Found
	FailureInputs.Add(
		TEXT(
			"{"
				"\"Object\":"
				"{"
					"\"NestedValue\":null,")
	);

	// Missing first brace
	FailureInputs.Add(
		TEXT(
			"\"Object\":"
			"{"
				"\"NestedValue\":null,"
				"\"NestedObject\":{}"
			"},"
		"\"Value\":true"
		"}"
		)
	);

	// Missing last character
	FailureInputs.Add(
		TEXT(
			"{"
				"\"Object\":"
				"{"
					"\"NestedValue\":null,"
					"\"NestedObject\":{}"
				"},"
				"\"Value\":true"
		)
	);

	// Missing curly brace
	FailureInputs.Add(TEXT("}"));

	// Missing bracket
	FailureInputs.Add(TEXT("]"));

	// Extra last character
	FailureInputs.Add(
		TEXT(
			"{"
				"\"Object\":"
				"{"
					"\"NestedValue\":null,"
					"\"NestedObject\":{}"
				"},"
				"\"Value\":true"
			"}0"
		)
	);

	// Missing comma
	FailureInputs.Add(
		TEXT(
			"{"
				"\"Value1\":null,"
				"\"Value2\":\"string\""
				"\"Value3\":65.3"
			"}"
		)
	);

	// Extra comma
	FailureInputs.Add(
		TEXT(
			"{"
				"\"Value1\":null,"
				"\"Value2\":\"string\","
				"\"Value3\":65.3,"
			"}"
		)
	);

	// Badly formed true/false/null
	FailureInputs.Add(TEXT("{\"Value\":tru}"));
	FailureInputs.Add(TEXT("{\"Value\":full}"));
	FailureInputs.Add(TEXT("{\"Value\":nulle}"));
	FailureInputs.Add(TEXT("{\"Value\":n%ll}"));

	// Floating Point Failures
	FailureInputs.Add(TEXT("{\"Value\":65.3e}"));
	FailureInputs.Add(TEXT("{\"Value\":65.}"));
	FailureInputs.Add(TEXT("{\"Value\":.7}"));
	FailureInputs.Add(TEXT("{\"Value\":+6}"));
	FailureInputs.Add(TEXT("{\"Value\":01}"));
	FailureInputs.Add(TEXT("{\"Value\":00.56}"));
	FailureInputs.Add(TEXT("{\"Value\":-1.e+4}"));
	FailureInputs.Add(TEXT("{\"Value\":2e+}"));

	// Bad Escape Characters
	FailureInputs.Add(TEXT("{\"Value\":\"Hello\\xThere\"}"));
	FailureInputs.Add(TEXT("{\"Value\":\"Hello\\u123There\"}"));
	FailureInputs.Add(TEXT("{\"Value\":\"Hello\\RThere\"}"));

	for (int32 i = 0; i < FailureInputs.Num(); ++i)
	{
		TSharedRef< TJsonReader<> > Reader = TJsonReaderFactory<>::Create( FailureInputs[i] );

		TSharedPtr<FJsonObject> Object;
		REQUIRE(FJsonSerializer::Deserialize( Reader, Object ) == false);
		REQUIRE(!Object.IsValid());
	}

	// TryGetNumber tests
	{
		auto JsonNumberToInt64 = [](double Val, int64& OutVal) -> bool
		{
			FJsonValueNumber JsonVal(Val);
			return ((FJsonValue&)JsonVal).TryGetNumber(OutVal);
		};

		auto JsonNumberToInt32 = [](double Val, int32& OutVal) -> bool
		{
			FJsonValueNumber JsonVal(Val);
			return ((FJsonValue&)JsonVal).TryGetNumber(OutVal);
		};

		auto JsonNumberToUInt32 = [](double Val, uint32& OutVal) -> bool
		{
			FJsonValueNumber JsonVal(Val);
			return ((FJsonValue&)JsonVal).TryGetNumber(OutVal);
		};
		
		// TryGetNumber-Int64 tests
		{
			int64 IntVal;
			bool bOk = JsonNumberToInt64(9007199254740991.0, IntVal);
			TestTrue(TEXT("TryGetNumber-Int64 Big Float64 succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-Int64 Big Float64"), IntVal, 9007199254740991LL);
		}

		{
			int64 IntVal;
			bool bOk = JsonNumberToInt64(-9007199254740991.0, IntVal);
			TestTrue(TEXT("TryGetNumber-Int64 Small Float64 succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-Int64 Small Float64"), IntVal, -9007199254740991LL);
		}

		{
			int64 IntVal;
			bool bOk = JsonNumberToInt64(0.4999999999999997, IntVal);
			TestTrue(TEXT("TryGetNumber-Int64 Lesser than near half succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-Int64 Lesser than near half rounds to zero"), IntVal, 0LL);
		}

		{
			int64 IntVal;
			bool bOk = JsonNumberToInt64(-0.4999999999999997, IntVal);
			TestTrue(TEXT("TryGetNumber-Int64 Greater than near negative half succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-Int64 Greater than near negative half rounds to zero"), IntVal, 0LL);
		}

		{
			int64 IntVal;
			bool bOk = JsonNumberToInt64(0.5, IntVal);
			TestTrue(TEXT("TryGetNumber-Int64 Half rounds to next integer succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-Int64 Half rounds to next integer"), IntVal, 1LL);
		}

		{
			int64 IntVal;
			bool bOk = JsonNumberToInt64(-0.5, IntVal);
			TestTrue(TEXT("TryGetNumber-Int64 Negative half rounds to next negative integer succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-Int64 Negative half rounds to next negative integer succeeds"), IntVal, -1LL);
		}

		// TryGetNumber-Int32 tests
		{
			int32 IntVal;
			bool bOk = JsonNumberToInt32(2147483647.000001, IntVal);
			TestFalse(TEXT("TryGetNumber-Int32 Number greater than max Int32 fails"), bOk);
		}

		{
			int32 IntVal;
			bool bOk = JsonNumberToInt32(-2147483648.000001, IntVal);
			TestFalse(TEXT("TryGetNumber-Int32 Number lesser than min Int32 fails"), bOk);
		}

		{
			int32 IntVal;
			bool bOk = JsonNumberToInt32(2147483647.0, IntVal);
			TestTrue(TEXT("TryGetNumber-Int32 Max Int32 succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-Int32 Max Int32"), IntVal, INT_MAX);
		}

		{
			int32 IntVal;
			bool bOk = JsonNumberToInt32(2147483646.5, IntVal);
			TestTrue(TEXT("TryGetNumber-Int32 Round up to max Int32 succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-Int32 Round up to max Int32"), IntVal, INT_MAX);
		}

		{
			int32 IntVal;
			bool bOk = JsonNumberToInt32(-2147483648.0, IntVal);
			TestTrue(TEXT("TryGetNumber-Int32 Min Int32 succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-Int32 Min Int32"), IntVal, INT_MIN);
		}

		{
			int32 IntVal;
			bool bOk = JsonNumberToInt32(-2147483647.5, IntVal);
			TestTrue(TEXT("TryGetNumber-Int32 Round down to min Int32 succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-Int32 Round down to min Int32"), IntVal, INT_MIN);
		}

		{
			int32 IntVal;
			bool bOk = JsonNumberToInt32(0.4999999999999997, IntVal);
			TestTrue(TEXT("TryGetNumber-Int32 Lesser than near half succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-Int32 Lesser than near half rounds to zero"), IntVal, 0);
		}

		{
			int32 IntVal;
			bool bOk = JsonNumberToInt32(-0.4999999999999997, IntVal);
			TestTrue(TEXT("TryGetNumber-Int32 Greater than near negative half succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-Int32 Greater than near negative half rounds to zero"), IntVal, 0);
		}

		{
			int32 IntVal;
			bool bOk = JsonNumberToInt32(0.5, IntVal);
			TestTrue(TEXT("TryGetNumber-Int32 Half rounds to next integer succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-Int32 Half rounds to next integer"), IntVal, 1);
		}

		{
			int32 IntVal;
			bool bOk = JsonNumberToInt32(-0.5, IntVal);
			TestTrue(TEXT("TryGetNumber-Int32 Negative half rounds to next negative integer succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-Int32 Negative half rounds to next negative integer succeeds"), IntVal, -1);
		}

		// TryGetNumber-UInt32 tests
		{
			uint32 IntVal;
			bool bOk = JsonNumberToUInt32(4294967295.000001, IntVal);
			TestFalse(TEXT("TryGetNumber-UInt32 Number greater than max Uint32 fails"), bOk);
		}

		{
			uint32 IntVal;
			bool bOk = JsonNumberToUInt32(-0.000000000000001, IntVal);
			TestFalse(TEXT("TryGetNumber-UInt32 Negative number fails"), bOk);
		}

		{
			uint32 IntVal;
			bool bOk = JsonNumberToUInt32(4294967295.0, IntVal);
			TestTrue(TEXT("TryGetNumber-UInt32 Max UInt32 succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-UInt32  Max UInt32"), IntVal, UINT_MAX);
		}

		{
			uint32 IntVal;
			bool bOk = JsonNumberToUInt32(4294967294.5, IntVal);
			TestTrue(TEXT("TryGetNumber-UInt32 Round up to max UInt32 succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-UInt32 Round up to max UInt32"), IntVal, UINT_MAX);
		}

		{
			uint32 IntVal;
			bool bOk = JsonNumberToUInt32(0.4999999999999997, IntVal);
			TestTrue(TEXT("TryGetNumber-UInt32 Lesser than near half succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-UInt32 Lesser than near half rounds to zero"), IntVal, 0U);
		}

		{
			uint32 IntVal;
			bool bOk = JsonNumberToUInt32(0.5, IntVal);
			TestTrue(TEXT("TryGetNumber-UInt32 Half rounds to next integer succeeds"), bOk);
			TestEqual(TEXT("TryGetNumber-UInt32 Half rounds to next integer"), IntVal, 1U);
		}
	}
}

TEST_CASE_NAMED(FJsonSerializerMacroTestSerializeMembersOf, "System::Engine::FileSystem::JSON::MacroToSerializeMembers", "[ApplicationContextMask][SmokeFilter]")
{
	struct FJsonStruct : public FJsonSerializable
	{
		struct FSubJsonStruct : public FJsonSerializable
		{
			int32 SubVarInt;
			FString SubVarString;

			FSubJsonStruct(int32 InSubVarInt = 0, FStringView InSubVarString = TEXTVIEW(""))
				: SubVarInt(InSubVarInt)
				, SubVarString(InSubVarString)
			{
			}

			BEGIN_JSON_SERIALIZER
				JSON_SERIALIZE("sub_var_int", SubVarInt);
				JSON_SERIALIZE("sub_var_string", SubVarString);
			END_JSON_SERIALIZER
		};

		int32 VarInt = 0;
		FSubJsonStruct VarSerializable;

		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE("var_int", VarInt);
			JSON_SERIALIZE_MEMBERS_OF(VarSerializable);
		END_JSON_SERIALIZER;
	};

	const TCHAR* SourceJsonString = TEXT("{\"var_int\":2,\"sub_var_int\":10,\"sub_var_string\":\"abc\"}");

	FJsonStruct JsonStructTarget;
	JsonStructTarget.FromJson(SourceJsonString);

	CHECK(JsonStructTarget.VarInt == 2);
	CHECK(JsonStructTarget.VarSerializable.SubVarInt == 10);
	CHECK(JsonStructTarget.VarSerializable.SubVarString == TEXT("abc"));

	FString ToJsonResult = JsonStructTarget.ToJson(/*bPrettyPrint*/false);
	CHECK(ToJsonResult == SourceJsonString);
}

TEST_CASE_NAMED(FJsonSerializerMacroTestSerializeSharedPtr, "System::Engine::FileSystem::JSON::MacroToSerializeSharedPtr", "[ApplicationContextMask][SmokeFilter]")
{
	struct FJsonStruct : public FJsonSerializable
	{
		struct FSubJsonStruct : public FJsonSerializable
		{
			int32 SubVarInt;
			FString SubVarString;

			FSubJsonStruct(int32 InSubVarInt = 0, FStringView InSubVarString = TEXTVIEW(""))
				: SubVarInt(InSubVarInt)
				, SubVarString(InSubVarString)
			{
			}

			BEGIN_JSON_SERIALIZER
				JSON_SERIALIZE("sub_var_int", SubVarInt);
				JSON_SERIALIZE("sub_var_string", SubVarString);
			END_JSON_SERIALIZER
		};

		int32 VarInt = 0;
		TSharedPtr<FSubJsonStruct> VarSerializable;

		BEGIN_JSON_SERIALIZER
			JSON_SERIALIZE("var_int", VarInt);
			JSON_SERIALIZE_SHARED_OBJECT_SERIALIZABLE("var_shared_ptr", VarSerializable);
		END_JSON_SERIALIZER;
	};

	const TCHAR* SourceJsonString = TEXT("{\"var_int\":2,\"var_shared_ptr\":{\"sub_var_int\":10,\"sub_var_string\":\"abc\"}}");

	FJsonStruct JsonStructTarget;
	JsonStructTarget.FromJson(SourceJsonString);

	CHECK(JsonStructTarget.VarInt == 2);
	CHECK(JsonStructTarget.VarSerializable->SubVarInt == 10);
	CHECK(JsonStructTarget.VarSerializable->SubVarString == TEXT("abc"));

	FString ToJsonResult = JsonStructTarget.ToJson(/*bPrettyPrint*/false);
	CHECK(ToJsonResult == SourceJsonString);
}

#endif // WITH_TESTS
