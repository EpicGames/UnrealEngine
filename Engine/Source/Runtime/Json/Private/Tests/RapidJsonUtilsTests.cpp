// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/ArrayView.h"
#if WITH_TESTS

#include "CoreMinimal.h"
#include "JsonUtils/JsonConversion.h"
#include "Misc/AutomationTest.h"
#include "Serialization/JsonSerializer.h"
#include "Tests/TestHarnessAdapter.h"

#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

/** Ensure RapidJsonUtils parses with the same quirks as the default Json loader, e.g. converting between types correctly **/
TEST_CASE_NAMED(FRapidJsonUtilsTests, "System::Engine::FileSystem::RapidJSONUtils::Basic", "[ApplicationContextMask][SmokeFilter]")
{
	const TCHAR* JsonText = TEXT(R"json(
{
	"on": true,
	"off": false,
	"zero": 0,
	"min_int32": -2147483648,
	"max_int32":  2147483647,
	"max_uint32": 4294967295,
	"min_int64": -9223372036854775808,
	"max_int64":  9223372036854775807,
	"max_uint64": 18446744073709551615,
	"double": 1.1,
	"big_double": 18446744073709551615.18446744073709551615,
	"string": "hello world",
	"object": {
		"value": 42
	},
	"array": [
		"aaa",
		"bbb",
		"ccc",
		"ddd"
	],	
	"null": null
}

)json");

	// check to make sure all values are parsed the same across RapidJsonUtils an default Json
	// convert to RapidJson

	using namespace UE::Json;

	TValueOrError<FDocument, FParseError> RapidJsonDocument = Parse(JsonText);
	FString ParseErrorMessage = RapidJsonDocument.HasError() ? RapidJsonDocument.GetError().CreateMessage(JsonText) : FString();
	REQUIRE_MESSAGE(ParseErrorMessage, RapidJsonDocument.HasValue());

	TOptional<FConstObject> Root = GetRootObject(RapidJsonDocument.GetValue());
	REQUIRE(Root.IsSet());

	TOptional<bool> OnField = GetBoolField(*Root, TEXT("on"));
	REQUIRE(OnField.IsSet());
	REQUIRE(*OnField == true);

	TOptional<bool> OffField = GetBoolField(*Root, TEXT("off"));
	REQUIRE(OffField.IsSet());
	REQUIRE(*OffField == false);

	TOptional<int32> ZeroField = GetInt32Field(*Root, TEXT("zero"));
	REQUIRE(ZeroField.IsSet());
	REQUIRE(*ZeroField == 0);

	TOptional<int32> MinInt32Field = GetInt32Field(*Root, TEXT("min_int32"));
	REQUIRE(MinInt32Field.IsSet());
	REQUIRE(*MinInt32Field == MIN_int32);

	TOptional<int32> MaxInt32Field = GetInt32Field(*Root, TEXT("max_int32"));
	REQUIRE(MaxInt32Field.IsSet());
	REQUIRE(*MaxInt32Field == MAX_int32);

	TOptional<uint32> MaxUint32Field = GetUint32Field(*Root, TEXT("max_uint32"));
	REQUIRE(MaxUint32Field.IsSet());
	REQUIRE(*MaxUint32Field == MAX_uint32);

	TOptional<int64> MinInt64Field = GetInt64Field(*Root, TEXT("min_int64"));
	REQUIRE(MinInt64Field.IsSet());
	REQUIRE(*MinInt64Field == MIN_int64);

	TOptional<int64> MaxInt64Field = GetInt64Field(*Root, TEXT("max_int64"));
	REQUIRE(MaxInt64Field.IsSet());
	REQUIRE(*MaxInt64Field == MAX_int64);

	TOptional<uint64> MaxUint64Field = GetUint64Field(*Root, TEXT("max_uint64"));
	REQUIRE(MaxUint64Field.IsSet());
	REQUIRE(*MaxUint64Field == MAX_uint64);

	TOptional<double> DoubleField = GetDoubleField(*Root, TEXT("double"));
	REQUIRE(DoubleField.IsSet());
	REQUIRE(*DoubleField == 1.1);

	TOptional<double> BigDoubleField = GetDoubleField(*Root, TEXT("big_double"));
	REQUIRE(BigDoubleField.IsSet());
	REQUIRE(*BigDoubleField == 18446744073709551615.18446744073709551615);

	TOptional<FStringView> StringField = GetStringField(*Root, TEXT("string"));
	REQUIRE(StringField.IsSet());
	REQUIRE(StringField->Equals(TEXT("hello world")));

	TOptional<FConstObject> ObjectField = GetObjectField(*Root, TEXT("object"));
	REQUIRE(StringField.IsSet());
	TOptional<int32> ObjectValueField = GetInt32Field(*ObjectField, TEXT("value"));
	REQUIRE(ObjectValueField.IsSet());
	REQUIRE(*ObjectValueField == 42);

	TOptional<FConstArray> ArrayField = GetArrayField(*Root, TEXT("array"));
	REQUIRE(ArrayField.IsSet());
	REQUIRE(ArrayField->Size() == 4);

	REQUIRE(HasNullField(*Root, TEXT("null")));

	REQUIRE(!GetBoolField(*Root, TEXT("missing")));
	REQUIRE(!GetInt32Field(*Root, TEXT("missing")));
	REQUIRE(!GetUint32Field(*Root, TEXT("missing")));
	REQUIRE(!GetInt64Field(*Root, TEXT("missing")));
	REQUIRE(!GetUint64Field(*Root, TEXT("missing")));
	REQUIRE(!GetDoubleField(*Root, TEXT("missing")));
	REQUIRE(!GetObjectField(*Root, TEXT("missing")));
	REQUIRE(!GetStringField(*Root, TEXT("missing")));
	REQUIRE(!GetArrayField(*Root, TEXT("missing")));
	REQUIRE(!HasNullField(*Root, TEXT("missing")));

}

/** Ensure RapidJsonUtils parses with the same quirks as the default Json loader, e.g. converting between types correctly **/
TEST_CASE_NAMED(FRapidJsonUtilsErrorHandlingTests, "System::Engine::FileSystem::RapidJSONUtils::ErrorHandling", "[ApplicationContextMask][SmokeFilter]")
{
	// check to make sure all values are parsed the same across RapidJsonUtils an default Json
	// convert to RapidJson

	using namespace UE::Json;

	// check newlines work correctly across all versions
	const TCHAR* JsonUnixText = TEXT("{\n\"ok\": 42,\n\"bad\": xyz\n}");
	const TCHAR* JsonMacText = TEXT("{\r\"ok\": 42,\r\"bad\": xyz\r}");
	const TCHAR* JsonWindowsText = TEXT("{\r\n\"ok\": 42,\r\n\"bad\": xyz\r\n}");

	TValueOrError<FDocument, FParseError> UnixDoc = Parse(JsonUnixText);
	REQUIRE(UnixDoc.HasError());
	FString UnixError = UnixDoc.GetError().CreateMessage(JsonUnixText);
	REQUIRE_MESSAGE(UnixError, UnixError.Contains(TEXT("Line 3")));

	TValueOrError<FDocument, FParseError> MacDoc = Parse(JsonMacText);
	REQUIRE(MacDoc.HasError());
	FString MacError = MacDoc.GetError().CreateMessage(JsonMacText);
	REQUIRE_MESSAGE(MacError, MacError.Contains(TEXT("Line 3")));

	TValueOrError<FDocument, FParseError> WindowsDoc = Parse(JsonWindowsText);
	REQUIRE(WindowsDoc.HasError());
	FString WindowsError = WindowsDoc.GetError().CreateMessage(JsonWindowsText);
	REQUIRE_MESSAGE(WindowsError, WindowsError.Contains(TEXT("Line 3")));

	const TCHAR* LongJsonText = TEXT(R"json({
	"array": [
		0,
		1,
		2,
		3,
		4,
		5,
		6,
		7,
		bad_format, 
		9,
		10,
		11
	]
})json");

	TValueOrError<FDocument, FParseError> LongDoc = Parse(LongJsonText);
	REQUIRE(LongDoc.HasError());
	FString LongError = LongDoc.GetError().CreateMessage(LongJsonText);

	// make sure the snippet contains our issue
	REQUIRE_MESSAGE(LongError, LongError.Contains(TEXT("bad_format")));

}

/** Check to make sure it doesn't crash with an empty input **/
TEST_CASE_NAMED(FRapidJsonUtilsParseEmpty, "System::Engine::FileSystem::RapidJSONUtils::ParseEmpty", "[ApplicationContextMask][SmokeFilter]")
{
	using namespace UE::Json;

	TValueOrError<FDocument, FParseError> Result = Parse({});

	REQUIRE(Result.HasError());
	REQUIRE(Result.GetError().ErrorCode == rapidjson::kParseErrorDocumentEmpty);

	TValueOrError<FDocument, FParseError> InPlaceResult = ParseInPlace({});
	REQUIRE(InPlaceResult.HasError());
	REQUIRE(InPlaceResult.GetError().ErrorCode == rapidjson::kParseErrorDocumentEmpty);
}

/** Check to make inputs aren't required to be null terminated **/
TEST_CASE_NAMED(FRapidJsonUtilsParseWithoutNullTerminator, "System::Engine::FileSystem::RapidJSONUtils::NoNullTerminator", "[ApplicationContextMask][SmokeFilter]")
{
	using namespace UE::Json;

	FString UnterminatedJsonText(TEXT(R"json(
		{ "key" : "value" };trailing
		)json"));

	int32 Index = 0;
	REQUIRE(UnterminatedJsonText.FindLastChar('}', Index));
	TArrayView<TCHAR> View = MakeArrayView(GetData(UnterminatedJsonText), Index+1);

	TValueOrError<FDocument, FParseError> Result = Parse(FStringView(View));
	REQUIRE(!Result.HasError());

	TValueOrError<FDocument, FParseError> InPlaceResult = ParseInPlace(MakeArrayView(View));
	REQUIRE(!InPlaceResult.HasError());
}


#endif // WITH_TESTS
