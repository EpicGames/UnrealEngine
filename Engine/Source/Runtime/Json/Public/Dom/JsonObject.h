// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/SharedString.h"
#include "Containers/StringFwd.h"
#include "CoreMinimal.h"
#include "Dom/JsonValue.h"
#include "HAL/Platform.h"
#include "JsonGlobals.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Serialization/JsonTypes.h"
#include "Templates/SharedPointer.h"
#include "Trace/Detail/Channel.h"

//
//
//----------------------------------------------------------------------------------------------------------
// 
// In the future, FJsonObject will only work with UE::FSharedString to store keys to values in JSON objects.
// This is done to save memory, so that often-recurring keys in json structures can reuse the same
// string allocations.
// 
// In UE::FSharedString mode, the interface of FJsonObject has been reworked to always take FStringView 
// instead of FString, since storing the key string is (a) no longer guaranteed to happen, and (b) now 
// done using UE::FSharedString. As a consequence, it's no longer possible to pass a `char*` as the key, 
// meaning that you can no longer pass "foobar", you have to pass TEXT("foobar").
// 
//----------------------------------------------------------------------------------------------------------
// 
// WORKAROUND:
// 
// To temporarily go back to the old FString interface, define UE_JSONOBJECT_LEGACY_STRING_KEYS=1 in your
// build. 
//----------------------------------------------------------------------------------------------------------
// 
// In a future release, UE_JSONOBJECT_LEGACY_STRING_KEYS will be removed, and UE::FSharedString will
// become the only supported string key type.
// 
//----------------------------------------------------------------------------------------------------------
//
//
#if !defined(UE_JSONOBJECT_LEGACY_STRING_KEYS)
#define UE_JSONOBJECT_LEGACY_STRING_KEYS 0
#endif

namespace UE::JSON::Private
{

#if !UE_JSONOBJECT_LEGACY_STRING_KEYS

	/**
	 * A set of shared strings that appear as keys in a FJsonObject.
	 * They are shared among the entire Json structure to reduce memory usage.
	 */
	class FJsonStringSet
	{
	public:
		JSON_API UE::FSharedString Place(FStringView Text);
		JSON_API SIZE_T GetAllocatedSize() const;
	private:
		/** Case sensitive hashing function for TSet */
		struct FCaseSensitiveStringKeyFuncs : BaseKeyFuncs<UE::FSharedString, UE::FSharedString>
		{
			static FORCEINLINE const UE::FSharedString& GetSetKey(const UE::FSharedString& Element)
			{
				return Element;
			}
			static FORCEINLINE bool Matches(const UE::FSharedString& A, const UE::FSharedString& B)
			{
				return A.ToView().Equals(B.ToView(), ESearchCase::CaseSensitive);
			}
			static uint32 GetKeyHash(const UE::FSharedString& Key);
		};

		TCompactSet<UE::FSharedString, FCaseSensitiveStringKeyFuncs> Strings;
	};


	/**
	 * Base class for UE::FSharedString based FJsonObject. This class will go away in a future release. Only use FJsonObject directly.
	 */
	class FJsonObjectSharedStringStorage 
	{
		TSharedPtr<UE::JSON::Private::FJsonStringSet> StringSet;
		TSharedPtr<UE::JSON::Private::FJsonStringSet> GetOrAddStringSet()
		{
			if (!StringSet)
			{
				StringSet = MakeShared<UE::JSON::Private::FJsonStringSet>();
			}
			return StringSet;
		}

	public:
		using FStringType = UE::FSharedString;

		/* Default constructor*/
		FJsonObjectSharedStringStorage() = default;

		/* Constructor taking a shared stringset that's shared with other json objects. */
		JSON_API explicit FJsonObjectSharedStringStorage(TSharedPtr<UE::JSON::Private::FJsonStringSet> StringSet);

		/**
		 *  Returns the stringset used to store shared key strings. Can be a null pointer if no keys were added yet.
		 */
		TSharedPtr<UE::JSON::Private::FJsonStringSet> GetStringSet() const
		{
			return StringSet;
		}

		/**
		 * Returns the memory footprint for this object in Bytes, including sizeof(*this) and allocated memory.
		 */
		SIZE_T GetMemoryFootprint() const
		{
			return sizeof(*this) + GetAllocatedSize();
		}

		/** Helper to calculate allocated size of the Values map and its contents */
		JSON_API SIZE_T GetAllocatedSize() const;

		/**
		 * Sets the generic json value of the field with the specified name.
		 * @param FieldName The name of the field to set.
		 * @param Value The value to set.
		 */
		JSON_API void SetField(FStringView FieldName, const TSharedPtr<FJsonValue>& Value);
		UE_DEPRECATED(5.8, "Use SetField(FStringView, ...)") UE_REWRITE void SetField(const ANSICHAR* FieldName, const TSharedPtr<FJsonValue>& Value)
		{
			SetField(FString(FieldName), Value);
		}

		/**
		 * Sets the number value of the field with the specified name.
		 * @param FieldName The name of the field to set.
		 * @param Value The value to set.
		 */
		JSON_API void SetNumberField(FStringView FieldName, double Number);
		UE_DEPRECATED(5.8, "Use SetNumberField(FStringView, ...)") UE_REWRITE void SetNumberField(const ANSICHAR* FieldName, double Number)
		{
			SetNumberField(FString(FieldName), Number);
		}

		/**
		 * Sets the string value of the field with the specified name.
		 * @param FieldName The name of the field to set.
		 * @param Value The value to set.
		 */
		JSON_API void SetStringField(FStringView FieldName, const ANSICHAR* StringValue);
		JSON_API void SetStringField(FStringView FieldName, const TCHAR* StringValue);
		JSON_API void SetStringField(FStringView FieldName, const UTF8CHAR* StringValue);
		JSON_API void SetStringField(FStringView FieldName, FString&& StringValue);
		JSON_API void SetStringField(FStringView FieldName, const FString& StringValue);
		JSON_API void SetStringField(FStringView FieldName, FUtf8String&& StringValue);
		JSON_API void SetStringField(FStringView FieldName, const FUtf8String& StringValue);
		JSON_API void SetStringField(FStringView FieldName, const UE::FSharedString& StringValue);
		UE_DEPRECATED(5.8, "Use SetStringField(FStringView, ...)") UE_REWRITE void SetStringField(const ANSICHAR* FieldName, const ANSICHAR* StringValue)
		{ 
			SetStringField(FString(FieldName), StringValue); 
		}
		UE_DEPRECATED(5.8, "Use SetStringField(FStringView, ...)") UE_REWRITE void SetStringField(const ANSICHAR* FieldName, const TCHAR* StringValue)
		{
			SetStringField(FString(FieldName), StringValue);
		}
		UE_DEPRECATED(5.8, "Use SetStringField(FStringView, ...)") UE_REWRITE void SetStringField(const ANSICHAR* FieldName, const UTF8CHAR* StringValue)
		{
			SetStringField(FString(FieldName), StringValue);
		}
		UE_DEPRECATED(5.8, "Use SetStringField(FStringView, ...)") UE_REWRITE void SetStringField(const ANSICHAR* FieldName, FString&& StringValue)
		{
			SetStringField(FString(FieldName), MoveTemp(StringValue));
		}
		UE_DEPRECATED(5.8, "Use SetStringField(FStringView, ...)") UE_REWRITE void SetStringField(const ANSICHAR* FieldName, const FString& StringValue)
		{
			SetStringField(FString(FieldName), CopyTemp(StringValue));
		}
		UE_DEPRECATED(5.8, "Use SetStringField(FStringView, ...)") UE_REWRITE void SetStringField(const ANSICHAR* FieldName, FUtf8String&& StringValue)
		{
			SetStringField(FString(FieldName), MoveTemp(StringValue));
		}
		UE_DEPRECATED(5.8, "Use SetStringField(FStringView, ...)") UE_REWRITE void SetStringField(const ANSICHAR* FieldName, const FUtf8String& StringValue)
		{
			SetStringField(FString(FieldName), CopyTemp(StringValue));
		}
		UE_DEPRECATED(5.8, "Use SetStringField(FStringView, ...)") UE_REWRITE void SetStringField(const ANSICHAR* FieldName, const UE::FSharedString& StringValue)
		{
			SetStringField(FString(FieldName), CopyTemp(StringValue));
		}

		/**
		 * Sets the bool value of the field with the specified name.
		 * @param FieldName The name of the field to set.
		 * @param Value The value to set.
		 */
		JSON_API void SetBoolField(FStringView FieldName, bool InValue);
		UE_DEPRECATED(5.8, "Use SetBoolField(FStringView, ...)") UE_REWRITE void SetBoolField(const ANSICHAR* FieldName, bool InValue)
		{
			SetBoolField(FString(FieldName), InValue);
		}

		/**
		 * Sets the array value of the field with the specified name.
		 * @param FieldName The name of the field to set.
		 * @param Value The value to set.
		 */
		JSON_API void SetArrayField(FStringView FieldName, TArray<TSharedPtr<FJsonValue>>&& Array);
		JSON_API void SetArrayField(FStringView FieldName, const TArray<TSharedPtr<FJsonValue>>& Array);
		UE_DEPRECATED(5.8, "Use SetArrayField(FStringView, ...)") UE_REWRITE void SetArrayField(const ANSICHAR* FieldName, TArray<TSharedPtr<FJsonValue>>&& Array)
		{
			SetArrayField(FString(FieldName), MoveTemp(Array));
		}
		UE_DEPRECATED(5.8, "Use SetArrayField(FStringView, ...)") UE_REWRITE void SetArrayField(const ANSICHAR* FieldName, const TArray<TSharedPtr<FJsonValue>>& Array)
		{
			SetArrayField(FString(FieldName), CopyTemp(Array));
		}

		/**
		 * Sets the object value of the field with the specified name.
		 * @param FieldName The name of the field to set.
		 * @param Value The value to set.
		 */
		JSON_API void SetObjectField(FStringView FieldName, TSharedPtr<FJsonObject>&& JsonObject);
		JSON_API void SetObjectField(FStringView FieldName, const TSharedPtr<FJsonObject>& JsonObject);
		UE_DEPRECATED(5.8, "Use SetObjectField(FStringView, ...)") UE_REWRITE void SetObjectField(const ANSICHAR* FieldName, TSharedPtr<FJsonObject>&& JsonObject)
		{
			SetObjectField(FString(FieldName), MoveTemp(JsonObject));
		}
		UE_DEPRECATED(5.8, "Use SetObjectField(FStringView, ...)") UE_REWRITE void SetObjectField(const ANSICHAR* FieldName, const TSharedPtr<FJsonObject>& JsonObject)
		{
			SetObjectField(FString(FieldName), CopyTemp(JsonObject));
		}

		TMap<FStringType, TSharedPtr<FJsonValue>> Values;
	};

#else

	/**
	 * Base class for FString based FJsonObject. This class will go away in a future release. Only use FJsonObject directly.
	 */
	class FJsonObjectStringStorage 
	{
	public:
		using FStringType = FString;

		/**
		 * Sets the value of the field with the specified name.
		 *
		 * @param FieldName The name of the field to set.
		 * @param Value The value to set.
		 */
		JSON_API void SetField(const UE::FSharedString& FieldName, const TSharedPtr<FJsonValue>& Value);
		JSON_API void SetField(FStringView FieldName, const TSharedPtr<FJsonValue>& Value);
		JSON_API void SetField(const FString& FieldName, const TSharedPtr<FJsonValue>& Value);
		JSON_API void SetField(const TCHAR* FieldName, const TSharedPtr<FJsonValue>& Value);

		/** Add a field named FieldName with Number as value */
		JSON_API void SetNumberField(FString&& FieldName, double Number);
		JSON_API void SetNumberField(const FString& FieldName, double Number);

		/** Add a field named FieldName with value of StringValue */
		JSON_API void SetStringField(FString&& FieldName, const ANSICHAR* StringValue);
		JSON_API void SetStringField(const FString& FieldName, const ANSICHAR* StringValue);
		JSON_API void SetStringField(FString&& FieldName, const TCHAR* StringValue);
		JSON_API void SetStringField(const FString& FieldName, const TCHAR* StringValue);
		JSON_API void SetStringField(FString&& FieldName, const UTF8CHAR* StringValue);
		JSON_API void SetStringField(const FString& FieldName, const UTF8CHAR* StringValue);

		JSON_API void SetStringField(FString&& FieldName, FString&& StringValue);
		JSON_API void SetStringField(FString&& FieldName, const FString& StringValue);
		JSON_API void SetStringField(const FString& FieldName, FString&& StringValue);
		JSON_API void SetStringField(const FString& FieldName, const FString& StringValue);
		JSON_API void SetStringField(FString&& FieldName, FUtf8String&& StringValue);
		JSON_API void SetStringField(FString&& FieldName, const FUtf8String& StringValue);
		JSON_API void SetStringField(const FString& FieldName, FUtf8String&& StringValue);
		JSON_API void SetStringField(const FString& FieldName, const FUtf8String& StringValue);

		/** Set a boolean field named FieldName and value of InValue */
		JSON_API void SetBoolField(FString&& FieldName, bool InValue);
		JSON_API void SetBoolField(const FString& FieldName, bool InValue);


		/** Set an array field named FieldName and value of Array */
		JSON_API void SetArrayField(FString&& FieldName, TArray<TSharedPtr<FJsonValue>>&& Array);
		JSON_API void SetArrayField(FString&& FieldName, const TArray<TSharedPtr<FJsonValue>>& Array);
		JSON_API void SetArrayField(const FString& FieldName, TArray<TSharedPtr<FJsonValue>>&& Array);
		JSON_API void SetArrayField(const FString& FieldName, const TArray<TSharedPtr<FJsonValue>>& Array);

		/** Set an ObjectField named FieldName and value of JsonObject */
		JSON_API void SetObjectField(FString&& FieldName, TSharedPtr<FJsonObject>&& JsonObject);
		JSON_API void SetObjectField(FString&& FieldName, const TSharedPtr<FJsonObject>& JsonObject);
		JSON_API void SetObjectField(const FString& FieldName, const TSharedPtr<FJsonObject>& JsonObject);

		/**
		 * Returns the memory footprint for this object in Bytes, including sizeof(*this) and allocated memory.
		 */
		SIZE_T GetMemoryFootprint() const
		{
			return sizeof(*this) + GetAllocatedSize();
		}

		/** Helper to calculate allocated size of the Values map and its contents */
		JSON_API SIZE_T GetAllocatedSize() const;

		TMap<FStringType, TSharedPtr<FJsonValue>> Values;
	};

#endif

}

/**
 * A Json Object is a structure holding an unordered set of name/value pairs.
 * Can use either FString (old) or UE::FSharedString (new) as the key type.
 * 
 * In a Json file, it is represented by everything between curly braces {}.
 */
class FJsonObject 
#if !UE_JSONOBJECT_LEGACY_STRING_KEYS
	: public UE::JSON::Private::FJsonObjectSharedStringStorage 
#else
	: public UE::JSON::Private::FJsonObjectStringStorage
#endif
{
public:
#if !UE_JSONOBJECT_LEGACY_STRING_KEYS
	using FJsonObjectSharedStringStorage::FJsonObjectSharedStringStorage;
#else
	using FJsonObjectStringStorage::FJsonObjectStringStorage;
#endif

	template<EJson JsonType>
	TSharedPtr<FJsonValue> GetField(FStringView FieldName) const
	{
		return GetField(FieldName, JsonType);
	}
	JSON_API TSharedPtr<FJsonValue> GetField(FStringView FieldName, EJson JsonType) const;

	/**
	 * Gets a shared pointer to a field by name, if it exists.
	 */
	JSON_API const TSharedPtr<FJsonValue> GetFieldUntyped(FStringView FieldName) const;

	/**
	 * Attempts to get the field with the specified name.
	 *
	 * @param FieldName The name of the field to get.
	 * @return A pointer to the field, or nullptr if the field doesn't exist.
	 */
	JSON_API TSharedPtr<FJsonValue> TryGetField(FStringView FieldName) const;

	/**
	 * Checks whether a field with the specified name exists in the object.
	 *
	 * @param FieldName The name of the field to check.
	 * @return true if the field exists, false otherwise.
	 */
	JSON_API bool HasField(FStringView FieldName) const;

	/**
	 * Checks whether a field with the specified name and type exists in the object.
	 *
	 * @tparam JsonType The type of the field to check.
	 * @param FieldName The name of the field to check.
	 * @return true if the field exists, false otherwise.
	 */
	template<EJson JsonType>
	bool HasTypedField(FStringView FieldName) const
	{
		return HasTypedField(FieldName, JsonType);
	}

	/**
	 * Checks whether a field with the specified name and type exists in the object.
	 *
	 * @param JsonType The type of the field to check.
	 * @param FieldName The name of the field to check.
	 * @return true if the field exists, false otherwise.
	 */
	JSON_API bool HasTypedField(FStringView FieldName, EJson JsonType) const;

	/**
	 * Removes the field with the specified name.
	 *
	 * @param FieldName The name of the field to remove.
	 */
	JSON_API void RemoveField(FStringView FieldName);

	/**
	 * Removes all fields the predicate returns true for.
	 */
	JSON_API void RemoveIf(TFunctionRef<bool(const FStringType&, TSharedPtr<const FJsonValue>)> Predicate);

	/**
	 * Gets the field with the specified name as a number.
	 *
	 * Ensures that the field is present and is of type Json number.
	 *
	 * @param FieldName The name of the field to get.
	 * @return The field's value as a number.
	 */
	JSON_API double GetNumberField(FStringView FieldName) const;

	/**
	 * Gets a numeric field and casts to an int32
	 */
	JSON_API int32 GetIntegerField(FStringView FieldName) const;

	/** Get the field named FieldName as a number. Returns false if it doesn't exist or cannot be converted. */
	JSON_API bool TryGetNumberField(FStringView FieldName, float& OutNumber) const;

	/** Get the field named FieldName as a number. Returns false if it doesn't exist or cannot be converted. */
	JSON_API bool TryGetNumberField(FStringView FieldName, double& OutNumber) const;

	/** Get the field named FieldName as a number, and makes sure it's within int8 range. Returns false if it doesn't exist or cannot be converted. */
	JSON_API bool TryGetNumberField(FStringView FieldName, int8& OutNumber) const;

	/** Get the field named FieldName as a number, and makes sure it's within int16 range. Returns false if it doesn't exist or cannot be converted. */
	JSON_API bool TryGetNumberField(FStringView FieldName, int16& OutNumber) const;

	/** Get the field named FieldName as a number, and makes sure it's within int32 range. Returns false if it doesn't exist or cannot be converted. */
	JSON_API bool TryGetNumberField(FStringView FieldName, int32& OutNumber) const;

	/** Get the field named FieldName as a number. Returns false if it doesn't exist or cannot be converted. */
	JSON_API bool TryGetNumberField(FStringView FieldName, int64& OutNumber) const;

	/** Get the field named FieldName as a number, and makes sure it's within uint8 range. Returns false if it doesn't exist or cannot be converted. */
	JSON_API bool TryGetNumberField(FStringView FieldName, uint8& OutNumber) const;

	/** Get the field named FieldName as a number, and makes sure it's within uint16 range. Returns false if it doesn't exist or cannot be converted. */
	JSON_API bool TryGetNumberField(FStringView FieldName, uint16& OutNumber) const;

	/** Get the field named FieldName as a number, and makes sure it's within uint32 range. Returns false if it doesn't exist or cannot be converted.  */
	JSON_API bool TryGetNumberField(FStringView FieldName, uint32& OutNumber) const;

	/** Get the field named FieldName as a number, and makes sure it's within uint64 range. Returns false if it doesn't exist or cannot be converted.  */
	JSON_API bool TryGetNumberField(FStringView FieldName, uint64& OutNumber) const;

	/** Get the field named FieldName as a string. */
	JSON_API FString GetStringField(FStringView FieldName) const;

	/** Get the field named FieldName as a UTF8 string. */
	JSON_API FUtf8String GetUtf8StringField(FStringView FieldName) const;

	/** Get the field named FieldName as a string. Returns false if it doesn't exist or cannot be converted. */
	JSON_API bool TryGetStringField(FStringView FieldName, FString& OutString) const;

	/** Get the field named FieldName as an array of strings. Returns false if it doesn't exist or any member cannot be converted. */
	JSON_API bool TryGetStringArrayField(FStringView FieldName, TArray<FString>& OutArray) const;

	/** Get the field named FieldName as an array of enums. Returns false if it doesn't exist or any member is not a string. */
	template<typename TEnum>
	bool TryGetEnumArrayField(FStringView FieldName, TArray<TEnum>& OutArray) const
	{
		TArray<FString> Strings;
		if (!TryGetStringArrayField(FieldName, Strings))
		{
			return false;
		}

		OutArray.Empty();
		for (const FString& String : Strings)
		{
			TEnum Value;
			if (LexTryParseString(Value, *String))
			{
				OutArray.Add(Value);
			}
		}
		return true;
	}

	/**
	 * Gets the field with the specified name as a boolean.
	 *
	 * Ensures that the field is present and is of type Json number.
	 *
	 * @param FieldName The name of the field to get.
	 * @return The field's value as a boolean.
	 */
	JSON_API bool GetBoolField(FStringView FieldName) const;

	/** Get the field named FieldName as a string. Returns false if it doesn't exist or cannot be converted. */
	JSON_API bool TryGetBoolField(FStringView FieldName, bool& OutBool) const;

	/** Get the field named FieldName as an array. */
	JSON_API const TArray<TSharedPtr<FJsonValue>>& GetArrayField(FStringView FieldName) const;

	/** Try to get the field named FieldName as an array, or return false if it's another type */
	JSON_API bool TryGetArrayField(FStringView FieldName, const TArray<TSharedPtr<FJsonValue>>*& OutArray) const;

	/**
	 * Gets the field with the specified name as a Json object.
	 *
	 * Ensures that the field is present and is of type Json object.
	 *
	 * @param FieldName The name of the field to get.
	 * @return The field's value as a Json object.
	 */
	JSON_API const TSharedPtr<FJsonObject>& GetObjectField(FStringView FieldName) const;

	/** Try to get the field named FieldName as an object, or return false if it's another type */
	JSON_API bool TryGetObjectField(FStringView FieldName, const TSharedPtr<FJsonObject>*& OutObject) const;

	JSON_API static void Duplicate(const TSharedPtr<const FJsonObject>& Source, const TSharedPtr<FJsonObject>& Dest);
	JSON_API static void Duplicate(const TSharedPtr<FJsonObject>& Source, TSharedPtr<FJsonObject>& Dest);
};
