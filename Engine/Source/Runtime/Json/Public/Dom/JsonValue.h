// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "Containers/Utf8String.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/CString.h"
#include "Serialization/JsonTypes.h"
#include "Templates/SharedPointer.h"

class FJsonObject;

/**
 * A Json Value is a structure that can be any of the Json Types.
 * It should never be used on its, only its derived types should be used.
 */
class FJsonValue
{
public:

	/** Returns this value as a double, logging an error and returning zero if this is not an Json Number */
	JSON_API double AsNumber() const;

	/** Returns this value as a string, logging an error and returning an empty string if not possible */
	JSON_API FString AsString() const;

	/** Returns this value as a utf8 string, logging an error and returning an empty string if not possible */
	JSON_API FUtf8String AsUtf8String() const;

	/** Returns this value as a boolean, logging an error and returning false if not possible */
	JSON_API bool AsBool() const;

	/** Returns this value as an array, logging an error and returning an empty array reference if not possible */
	JSON_API const TArray<TSharedPtr<FJsonValue>>& AsArray() const;

	/** Returns this value as an object, throwing an error if this is not an Json Object */
	JSON_API virtual const TSharedPtr<FJsonObject>& AsObject() const;

	/** Tries to convert this value to a number, returning false if not possible */
	JSON_API virtual bool TryGetNumber(double& OutNumber) const;

	/** Tries to convert this value to a number, returning false if not possible */
	JSON_API virtual bool TryGetNumber(float& OutNumber) const;

	/** Tries to convert this value to a number, returning false if not possible */
	JSON_API virtual bool TryGetNumber(int8& OutNumber) const;

	/** Tries to convert this value to a number, returning false if not possible */
	JSON_API virtual bool TryGetNumber(int16& OutNumber) const;

	/** Tries to convert this value to a number, returning false if not possible */
	JSON_API virtual bool TryGetNumber(int32& OutNumber) const;

	/** Tries to convert this value to a number, returning false if not possible */
	JSON_API virtual bool TryGetNumber(int64& OutNumber) const;

	/** Tries to convert this value to a number, returning false if not possible */
	JSON_API virtual bool TryGetNumber(uint8& OutNumber) const;

	/** Tries to convert this value to a number, returning false if not possible */
	JSON_API virtual bool TryGetNumber(uint16& OutNumber) const;

	/** Tries to convert this value to a number, returning false if not possible */
	JSON_API virtual bool TryGetNumber(uint32& OutNumber) const;

	/** Tries to convert this value to a number, returning false if not possible */
	JSON_API virtual bool TryGetNumber(uint64& OutNumber) const;

	/** Tries to convert this value to a string, returning false if not possible */
	JSON_API virtual bool TryGetString(FString& OutString) const;

	/** Tries to convert this value to a utf8 string, returning false if not possible */
	JSON_API virtual bool TryGetUtf8String(FUtf8String& OutString) const;

	/** Tries to convert this value to a bool, returning false if not possible */
	JSON_API virtual bool TryGetBool(bool& OutBool) const;

	/** Tries to convert this value to an array, returning false if not possible */
	JSON_API virtual bool TryGetArray(const TArray<TSharedPtr<FJsonValue>>*& OutArray) const;
	
	/** Tries to convert this value to an array, returning false if not possible */
	JSON_API virtual bool TryGetArray(TArray<TSharedPtr<FJsonValue>>*& OutArray);

	/** Tries to convert this value to an object, returning false if not possible */
	JSON_API virtual bool TryGetObject(const TSharedPtr<FJsonObject>*& Object) const;

	/** Tries to convert this value to an object, returning false if not possible */
	JSON_API virtual bool TryGetObject(TSharedPtr<FJsonObject>*& Object);

	/** Returns whether or not a caller should prefer a string representation of the value, rather than the natural JSON type */
	JSON_API virtual bool PreferStringRepresentation() const;

	/** Returns true if this value is a 'null' */
	bool IsNull() const
	{
		return Type == EJson::Null || Type == EJson::None;
	}

	/** Get a field of the same type as the argument */
	void AsArgumentType(double& Value)
	{
		Value = AsNumber();
	}
	
	void AsArgumentType(FString& Value)
	{
		Value = AsString();
	}
	
	void AsArgumentType(bool& Value)
	{
		Value = AsBool();
	}
	
	void AsArgumentType(TArray<TSharedPtr<FJsonValue>>& Value)
	{
		Value = AsArray();
	}
	void AsArgumentType(TSharedPtr<FJsonObject>& Value)
	{
		Value = AsObject();
	}

	/**
	 * Returns the memory footprint for this object in Bytes, including sizeof(*this) and allocated memory.
	 * All children should implement this so their memory layout is properly accounted for
	 */
	JSON_API virtual SIZE_T GetMemoryFootprint() const;

	EJson Type;

	static JSON_API TSharedPtr<FJsonValue> Duplicate(const TSharedPtr<const FJsonValue>& Src);
	static JSON_API TSharedPtr<FJsonValue> Duplicate(const TSharedPtr<FJsonValue>& Src);

	static JSON_API bool CompareEqual(const FJsonValue& Lhs, const FJsonValue& Rhs);

protected:
	UE_NONCOPYABLE(FJsonValue)

	JSON_API FJsonValue();
	JSON_API virtual ~FJsonValue();

	virtual FString GetType() const = 0;

	JSON_API void ErrorMessage(const FString& InType) const;

	friend inline bool operator==(const FJsonValue& Lhs, const FJsonValue& Rhs)
	{
		return FJsonValue::CompareEqual(Lhs, Rhs);
	}

	friend inline bool operator!=(const FJsonValue& Lhs, const FJsonValue& Rhs)
	{
		return !FJsonValue::CompareEqual(Lhs, Rhs);
	}
};

/** A Json String Value. */
template <typename CharType>
class TJsonValueString : public FJsonValue
{
public:
	inline TJsonValueString(const TString<CharType>& InString);
	inline TJsonValueString(TString<CharType>&& InString);

	inline virtual bool TryGetString(FString& OutString) const override;	
	inline virtual bool TryGetUtf8String(FUtf8String& OutString) const override;
	inline virtual bool TryGetNumber(double& OutDouble) const override;
	inline virtual bool TryGetNumber(int32& OutValue) const override;	
	inline virtual bool TryGetNumber(uint32& OutValue) const override;
	inline virtual bool TryGetNumber(int64& OutValue) const override;
	inline virtual bool TryGetNumber(uint64& OutValue) const override;
	inline virtual bool TryGetBool(bool& OutBool) const override;

	// Way to check if string value is empty without copying the string 
	inline bool IsEmpty() const
	{
		return Value.IsEmpty();
	}

	inline virtual SIZE_T GetMemoryFootprint() const override;

protected:
	TString<CharType> Value;

	inline virtual FString GetType() const override;

	/** Helper to calculate allocated size of the Value string */
	inline SIZE_T GetAllocatedSize() const;
};

using FJsonValueString = TJsonValueString<TCHAR>;

//** A Json Number Value. */
class FJsonValueNumber : public FJsonValue
{
public:
	JSON_API FJsonValueNumber(double InNumber);
	JSON_API virtual ~FJsonValueNumber();
;
	JSON_API virtual bool TryGetNumber(double& OutNumber) const override;
	JSON_API virtual bool TryGetBool(bool& OutBool) const override;
	JSON_API virtual bool TryGetString(FString& OutString) const override;
	JSON_API virtual bool TryGetUtf8String(FUtf8String& OutString) const override;
	JSON_API virtual SIZE_T GetMemoryFootprint() const override;

protected:
	double Value;

	JSON_API virtual FString GetType() const override;
};

/** A Json Number Value, stored internally as a string so as not to lose precision */
template <typename CharType>
class TJsonValueNumberString : public FJsonValue
{
public:
	inline TJsonValueNumberString(const TString<CharType>& InString);
	inline TJsonValueNumberString(TString<CharType>&& InString);

	inline virtual bool TryGetString(FString& OutString) const override;	
	inline virtual bool TryGetUtf8String(FUtf8String& OutString) const override;
	inline virtual bool TryGetNumber(double& OutDouble) const override;
	inline virtual bool TryGetNumber(float &OutDouble) const override;
	inline virtual bool TryGetNumber(int8& OutValue) const override;
	inline virtual bool TryGetNumber(int16& OutValue) const override;
	inline virtual bool TryGetNumber(int32& OutValue) const override;
	inline virtual bool TryGetNumber(int64& OutValue) const override;
	inline virtual bool TryGetNumber(uint8& OutValue) const override;
	inline virtual bool TryGetNumber(uint16& OutValue) const override;
	inline virtual bool TryGetNumber(uint32& OutValue) const override;
	inline virtual bool TryGetNumber(uint64& OutValue) const override;
	inline virtual bool TryGetBool(bool& OutBool) const override;
	inline virtual bool PreferStringRepresentation() const override;
	inline virtual SIZE_T GetMemoryFootprint() const override;

protected:
	TString<CharType> Value;

	inline virtual FString GetType() const override;

	/** Helper to calculate allocated size of the Value string */
	inline SIZE_T GetAllocatedSize() const;
};

using FJsonValueNumberString = TJsonValueNumberString<TCHAR>;

/** A Json Boolean Value. */
class FJsonValueBoolean : public FJsonValue
{
public:
	JSON_API FJsonValueBoolean(bool InBool);
	JSON_API virtual ~FJsonValueBoolean();

	JSON_API virtual bool TryGetNumber(double& OutNumber) const override;
	JSON_API virtual bool TryGetBool(bool& OutBool) const override;
	JSON_API virtual bool TryGetString(FString& OutString) const override;
	JSON_API virtual bool TryGetUtf8String(FUtf8String& OutString) const override;
	JSON_API virtual SIZE_T GetMemoryFootprint() const override;

protected:
	bool Value;

	JSON_API virtual FString GetType() const override;
};

/** A Json Array Value. */
class FJsonValueArray : public FJsonValue
{
public:
	JSON_API FJsonValueArray(const TArray<TSharedPtr<FJsonValue>>& InArray);
	JSON_API FJsonValueArray(TArray<TSharedPtr<FJsonValue>>&& InArray);
	JSON_API virtual ~FJsonValueArray();

	JSON_API virtual bool TryGetArray(const TArray<TSharedPtr<FJsonValue>>*& OutArray) const override;
	JSON_API virtual bool TryGetArray(TArray<TSharedPtr<FJsonValue>>*& OutArray) override;
	JSON_API virtual SIZE_T GetMemoryFootprint() const override;

protected:
	TArray<TSharedPtr<FJsonValue>> Value;

	JSON_API virtual FString GetType() const override;	
/** Helper to calculate allocated size of the Value array and its contents */
	JSON_API SIZE_T GetAllocatedSize() const;
};


/** A Json Object Value. */
class FJsonValueObject : public FJsonValue
{
public:
	JSON_API FJsonValueObject(TSharedPtr<FJsonObject> InObject);
	JSON_API virtual ~FJsonValueObject();

	JSON_API virtual bool TryGetObject(const TSharedPtr<FJsonObject>*& OutObject) const override;
	JSON_API virtual bool TryGetObject(TSharedPtr<FJsonObject>*& OutObject) override;
	JSON_API virtual SIZE_T GetMemoryFootprint() const override;
protected:
	TSharedPtr<FJsonObject> Value;

	JSON_API virtual FString GetType() const override;
	/** Helper to calculate allocated size of the Value object and its contents */
	JSON_API SIZE_T GetAllocatedSize() const;
};


/** A Json Null Value. */
class FJsonValueNull : public FJsonValue
{
public:
	JSON_API FJsonValueNull();
	JSON_API virtual ~FJsonValueNull();

	JSON_API virtual SIZE_T GetMemoryFootprint() const override;

protected:
	JSON_API virtual FString GetType() const override;
};

namespace UE::Json
{

template<typename T, typename = typename std::enable_if<!std::is_same_v<T, FJsonValue>>>
inline JsonSimpleValueVariant ToSimpleJsonVariant(const T& InSimpleValue)
{
	using InSimpleValueType = std::decay_t<decltype(InSimpleValue)>;
	if constexpr (std::is_same_v<InSimpleValueType, bool> || std::is_same_v<InSimpleValueType, FString>)
	{
		return JsonSimpleValueVariant(TInPlaceType<T>(), InSimpleValue);
	}
	else
	{
		return JsonSimpleValueVariant(TInPlaceType<JsonNumberValueVariants>(), JsonNumberValueVariants(TInPlaceType<T>(), InSimpleValue));
	}
}

JSON_API JsonSimpleValueVariant ToSimpleJsonVariant(const FJsonValue& InJsonValue);

} // namespace UE::Json

JSON_API FString ToString(const JsonNumberValueVariants& InNumberVariant);

/* Global operators */
JSON_API bool operator==(const JsonSimpleValueVariant& Lhs, const JsonSimpleValueVariant& Rhs);
JSON_API bool operator==(const JsonNumberValueVariants& Lhs, const JsonNumberValueVariants& Rhs);
JSON_API bool operator==(const JsonNumberValueVariants& Lhs, const FString& Rhs);


#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS

inline bool operator==(const FString& Lhs, const JsonNumberValueVariants& Rhs)
{
	return Rhs == Lhs;
}

inline bool operator!=(const JsonNumberValueVariants& Lhs, const FString& Rhs)
{
	return !(Lhs == Rhs);
}

inline bool operator!=(const FString& Lhs, const JsonNumberValueVariants& Rhs)
{
	return !(Lhs == Rhs);
}

inline bool operator!=(const JsonNumberValueVariants& Lhs, const JsonNumberValueVariants& Rhs)
{
	return !(Lhs == Rhs);
}

inline bool operator!=(const JsonSimpleValueVariant& Lhs, const JsonSimpleValueVariant& Rhs)
{
	return !(Lhs == Rhs);
}
#endif

namespace
{
	template <typename CharType>
	struct TJsonValueStringType
	{
		static_assert(sizeof(CharType) == 0, "Unsupported type");
	};

	template <>
	struct TJsonValueStringType<TCHAR>
	{
		static FString GetType()
		{
			return TEXT("String");
		}
	};

	template <>
	struct TJsonValueStringType<UTF8CHAR>
	{
		static FString GetType()
		{
			return TEXT("Utf8String");
		}
	};
}

template <typename CharType>
TJsonValueString<CharType>::TJsonValueString(const TString<CharType>& InString)
	: Value(InString)
{
	Type = EJson::String;
}

template <typename CharType>
TJsonValueString<CharType>::TJsonValueString(TString<CharType>&& InString)
	: Value(MoveTemp(InString))
{
	Type = EJson::String;
}

template <typename CharType>
bool TJsonValueString<CharType>::TryGetString(FString& OutString) const
{
	if constexpr (std::is_same_v<FString, TString<CharType>>)
	{
		OutString = Value;
	}
	else
	{
		OutString = FString(Value);
	}
	return true;
}

template <typename CharType>
bool TJsonValueString<CharType>::TryGetUtf8String(FUtf8String& OutString) const
{
	if constexpr (std::is_same_v<FUtf8String, TString<CharType>>)
	{
		OutString = Value;
	}
	else
	{
		OutString = FUtf8String(Value);
	}
	return true;
}

template <typename CharType>
bool TJsonValueString<CharType>::TryGetNumber(double& OutDouble) const
{
	if (Value.IsNumeric())
	{
		OutDouble = TCString<CharType>::Atod(*Value);
		return true;
	}
	else
	{
		return false;
	}
}

template <typename CharType>
bool TJsonValueString<CharType>::TryGetNumber(int32& OutValue) const
{
	LexFromString(OutValue, *Value);
	return true;
}

template <typename CharType>
bool TJsonValueString<CharType>::TryGetNumber(uint32& OutValue) const
{
	LexFromString(OutValue, *Value);
	return true;
}

template <typename CharType>
bool TJsonValueString<CharType>::TryGetNumber(int64& OutValue) const
{
	LexFromString(OutValue, *Value);
	return true;
}

template <typename CharType>
bool TJsonValueString<CharType>::TryGetNumber(uint64& OutValue) const
{
	LexFromString(OutValue, *Value);
	return true;
}

template <typename CharType>
bool TJsonValueString<CharType>::TryGetBool(bool& OutBool) const
{
	OutBool = Value.ToBool();
	return true;
}

template <typename CharType>
SIZE_T TJsonValueString<CharType>::GetMemoryFootprint() const
{
	return sizeof(*this) + GetAllocatedSize();
}

template <typename CharType>
FString TJsonValueString<CharType>::GetType() const
{
	return TJsonValueStringType<CharType>::GetType();
}

template <typename CharType>
SIZE_T TJsonValueString<CharType>::GetAllocatedSize() const
{
	return Value.GetAllocatedSize();
}

namespace
{
	template <typename CharType>
	struct TJsonValueNumberType
	{
		static_assert(sizeof(CharType), "Unsupported type");
	};

	template <>
	struct TJsonValueNumberType<TCHAR>
	{
		static FString GetType()
		{
			return TEXT("NumberString");
		}
	};

	template <>
	struct TJsonValueNumberType<UTF8CHAR>
	{
		static FString GetType()
		{
			return TEXT("Utf8NumberString");
		}
	};
}

template <typename CharType>
TJsonValueNumberString<CharType>::TJsonValueNumberString(const TString<CharType>& InString)
	: Value(InString)
{
	Type = EJson::Number;
}

template <typename CharType>
TJsonValueNumberString<CharType>::TJsonValueNumberString(TString<CharType>&& InString)
	: Value(MoveTemp(InString))
{
	Type = EJson::Number;
}

template <typename CharType>
bool TJsonValueNumberString<CharType>::TryGetString(FString& OutString) const
{
	if constexpr (std::is_same_v<FString, TString<CharType>>)
	{
		OutString = Value;
	}
	else
	{
		OutString = FString(Value);
	}
	return true;
}

template <typename CharType>
bool TJsonValueNumberString<CharType>::TryGetUtf8String(FUtf8String& OutString) const
{
	if constexpr (std::is_same_v<FUtf8String, TString<CharType>>)
	{
		OutString = Value;
	}
	else
	{
		OutString = FUtf8String(Value);
	}
	return true;
}

template <typename CharType>
bool TJsonValueNumberString<CharType>::TryGetNumber(double& OutDouble) const
{
	return LexTryParseString(OutDouble, *Value);
}

template <typename CharType>
bool TJsonValueNumberString<CharType>::TryGetNumber(float& OutDouble) const
{
	return LexTryParseString(OutDouble, *Value);
}

template <typename CharType>
bool TJsonValueNumberString<CharType>::TryGetNumber(int8& OutValue) const
{
	return LexTryParseString(OutValue, *Value);
}

template <typename CharType>
bool TJsonValueNumberString<CharType>::TryGetNumber(int16& OutValue) const
{
	return LexTryParseString(OutValue, *Value);
}

template <typename CharType>
bool TJsonValueNumberString<CharType>::TryGetNumber(int32& OutValue) const
{
	return LexTryParseString(OutValue, *Value);
}

template <typename CharType>
bool TJsonValueNumberString<CharType>::TryGetNumber(int64& OutValue) const
{
	return LexTryParseString(OutValue, *Value);
}

template <typename CharType>
bool TJsonValueNumberString<CharType>::TryGetNumber(uint8& OutValue) const
{
	return LexTryParseString(OutValue, *Value);
}

template <typename CharType>
bool TJsonValueNumberString<CharType>::TryGetNumber(uint16& OutValue) const
{
	return LexTryParseString(OutValue, *Value);
}

template <typename CharType>
bool TJsonValueNumberString<CharType>::TryGetNumber(uint32& OutValue) const
{
	return LexTryParseString(OutValue, *Value);
}

template <typename CharType>
bool TJsonValueNumberString<CharType>::TryGetNumber(uint64& OutValue) const
{
	return LexTryParseString(OutValue, *Value);
}

template <typename CharType>
bool TJsonValueNumberString<CharType>::TryGetBool(bool& OutBool) const
{
	OutBool = Value.ToBool();
	return true;
}

template <typename CharType>
bool TJsonValueNumberString<CharType>::PreferStringRepresentation() const
{
	return true;
}

template <typename CharType>
SIZE_T TJsonValueNumberString<CharType>::GetMemoryFootprint() const
{
	return sizeof(*this) + GetAllocatedSize();
}

template <typename CharType>
FString TJsonValueNumberString<CharType>::GetType() const
{
	return TJsonValueNumberType<CharType>::GetType();
}

template <typename CharType>
SIZE_T TJsonValueNumberString<CharType>::GetAllocatedSize() const
{
	return Value.GetAllocatedSize();
}
