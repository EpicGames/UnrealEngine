// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()


/* Public dependencies
 *****************************************************************************/

#include "Core.h"


/* Public includes
 *****************************************************************************/

#include "JsonGlobals.h"

#include "Policies/JsonPrintPolicy.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Policies/CondensedJsonPrintPolicy.h"

#include "Serialization/JsonTypes.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"

#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonSerializerMacros.h"
