// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

JSON_API DECLARE_LOG_CATEGORY_EXTERN(LogJson, Log, All);
