// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "JsonUtils/RapidJsonUtils.h"

class FJsonValue;
class FJsonObject;

namespace UE {
namespace Json {

/**
 * Converts from a RapidJSON value to a shared json value
 */
JSON_API TSharedPtr<FJsonValue> ConvertRapidJsonToSharedJsonValue(const FValue& Value);

/**
 * Converts from a RapidJSON object to a shared json object
 */
JSON_API TSharedPtr<FJsonObject> ConvertRapidJsonToSharedJsonObject(FConstObject Object);

/**
 * Converts from shared JSON object to a RapidJSON Document.
 */
JSON_API TOptional<FDocument> ConvertSharedJsonToRapidJsonDocument(const FJsonObject& SrcObject);

}
}
