// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"

/**
 * Utility to update an array of json objects from an array of elements (of arbitrary type).
 * Elements in the source array and the destination json object array are matched based on an
 * arbitrary key (provided by the FGetElementKey and FTryGetJsonObjectKey delegates respectively).
 * Existing elements get "updated" via the FUpdateJsonObject delegate. The update scheme is entirely 
 * customizable; for example, it can be non-destructive and leave some json fields unchanged.
 * Elements from the source array that are not in the json array (based on the "key") are added to it.
 * Elements that are not present in the source array (based on the "key") are removed from the json array.
 * If the source array is empty the json array field is removed.
 */
template <typename ElementType, typename KeyType>
struct FJsonObjectArrayUpdater
{
	DECLARE_DELEGATE_RetVal_OneParam(KeyType, FGetElementKey, const ElementType&);

	DECLARE_DELEGATE_RetVal_TwoParams(bool, FTryGetJsonObjectKey, const FJsonObject&, KeyType& /*OutKey*/);

	DECLARE_DELEGATE_TwoParams(FUpdateJsonObject, const ElementType&, FJsonObject&);

	DECLARE_DELEGATE_OneParam(FSortArray, TArray<TSharedPtr<FJsonValue>>&);

	static void Execute(FJsonObject& JsonObject, const FString& ArrayName, const TArray<ElementType>& SourceArray, FGetElementKey GetElementKey, FTryGetJsonObjectKey TryGetJsonObjectKey, FUpdateJsonObject UpdateJsonObject, FSortArray SortJsonArray = FSortArray())
	{
		if (SourceArray.Num() > 0)
		{
			TArray<TSharedPtr<FJsonValue>> NewJsonValues;
			{
				const TArray<TSharedPtr<FJsonValue>>* ExistingJsonValues;
				if (JsonObject.TryGetArrayField(ArrayName, ExistingJsonValues))
				{
					// Build a map of elements for quick access and to keep track of which ones got updated
					TMap<KeyType, const ElementType*> ElementsMap;
					for (const ElementType& Element : SourceArray)
					{
						ElementsMap.Add(GetElementKey.Execute(Element), &Element);
					}

					// Update existing json values and discard entries that no longer exist or are invalid
					for (TSharedPtr<FJsonValue> ExistingJsonValue : *ExistingJsonValues)
					{
						const TSharedPtr<FJsonObject>* ExistingJsonValueAsObject;
						if (ExistingJsonValue->TryGetObject(ExistingJsonValueAsObject))
						{
							KeyType ElementKey;
							if (TryGetJsonObjectKey.Execute(**ExistingJsonValueAsObject, ElementKey))
							{
								if (const ElementType** ElementPtr = ElementsMap.Find(ElementKey))
								{
									UpdateJsonObject.Execute(**ElementPtr, **ExistingJsonValueAsObject);
									NewJsonValues.Add(ExistingJsonValue);
									ElementsMap.Remove(ElementKey);
								}
							}
						}
					}

					// Add new elements
					for (auto It = ElementsMap.CreateConstIterator(); It; ++It)
					{
						TSharedPtr<FJsonObject> NewJsonObject = MakeShareable(new FJsonObject);
						UpdateJsonObject.Execute(*It.Value(), *NewJsonObject.Get());
						NewJsonValues.Add(MakeShareable(new FJsonValueObject(NewJsonObject)));
					}
				}
				else
				{
					// Array doesn't exist in the given JsonObject, so build a new array
					for (const ElementType& Element : SourceArray)
					{
						TSharedPtr<FJsonObject> NewJsonObject = MakeShareable(new FJsonObject);
						UpdateJsonObject.Execute(Element, *NewJsonObject.Get());
						NewJsonValues.Add(MakeShareable(new FJsonValueObject(NewJsonObject)));
					}
				}
			}

			// Optionally sort new json array before setting incase the user doesn't want the default
			// appending of new items at the end.
			SortJsonArray.ExecuteIfBound(NewJsonValues);

			// Set the new content of the json array
			JsonObject.SetArrayField(ArrayName, NewJsonValues);
		}
		else
		{
			// Source array is empty so remove the json array
			JsonObject.RemoveField(ArrayName);
		}
	}
};