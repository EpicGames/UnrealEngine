// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Dom/JsonValue.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

class FJsonObject;
class FJsonValue;

namespace UE::Json
{
	/**
	* JSON Pointer defines a string syntax for identifying a specific value
	* within a JavaScript Object Notation (JSON) document (from rfc6901).
	* @see https://datatracker.ietf.org/doc/html/rfc6901
	* @note Array append ("-") is currently unsupported.
	*/
	class FJsonPointer
	{
	public:
		FJsonPointer() = default;
		JSON_API FJsonPointer(FStringView Path);
		JSON_API FJsonPointer(const FJsonPointer& Other);
		JSON_API FJsonPointer(FJsonPointer&& Other) noexcept;

		/** Attempt to retrieve the value specified by this Json Pointer from the provided Json Value. */
		JSON_API bool TryGet(const TSharedPtr<FJsonObject>& RootObject, TSharedPtr<FJsonValue>& OutValue) const;

		/** Attempt to retrieve the value specified by this Json Pointer from the provided Json Object. */
		JSON_API bool TryGet(const TSharedPtr<FJsonValue>& RootValue, TSharedPtr<FJsonValue>& OutValue) const;

		/** Attempt to retrieve the value specified by this Json Pointer from the provided Json Array. */
		JSON_API bool TryGet(const TArray<TSharedPtr<FJsonValue>>& RootValues, TSharedPtr<FJsonValue>& OutValue) const;

		/** Checks that the path has one or more parts. */
		inline bool IsValid() const
		{
			return Length() > 0;
		}

		/** The number of parts in the path. */
		inline int32 Length() const
		{
			return PathParts.Num();
		}

		/** Appends the given part to the path */
		JSON_API void Append(FStringView Part);

		/** Sets the last path part to the given array index.  */
		JSON_API void SetArrayIndex(int32 Index);

		/** Gets the sub-path up to the given part length. */
		JSON_API FJsonPointer GetSubPath(int32 NumParts) const;

		/** Get the JsonPointer's string representation. */
		JSON_API FString ToString() const;

		/** Get the path part at the given index. */
		inline const FString& operator[](int32 Idx) const
		{
			return PathParts[Idx];
		}

		/** Get the path parts as an array. */
		inline const TArray<FString>& GetAll() const
		{
			return PathParts;
		}

		/** Substitutes characters in the provided Part with their escaped counterparts (as per the spec). */
		static JSON_API FString EscapePart(const FString& Part);
		
		/** Substitutes escaped characters in the Provided path with their original character (as per the spec). */
		static JSON_API FString UnescapePart(const FString& Part);

	private:
		/** Parses the provided path, populating PathParts and validating the result. */
		JSON_API bool ParsePath(const FString& InPath);

	private:
		TArray<FString> PathParts;
	};
}
