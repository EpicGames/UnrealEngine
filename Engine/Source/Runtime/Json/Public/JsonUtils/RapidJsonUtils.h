// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RapidJsonIncludes.h"
#include "Misc/Optional.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Templates/ValueOrError.h"

/**
 * Helper functions for reading/writing RapidJSON with unreal types
 */
namespace UE::Json
{
namespace Private
{
	// TODO: Can remove this and just use rapidjson::CrtAllocator once RapidJsonIncludes.h starts overriding RAPIDJSON_MALLOC, etc
	class FAllocatorImpl
	{
	public:
		static const bool kNeedFree = true;

		void* Malloc(size_t Count)
		{
			return FMemory::Malloc(Count);
		}

		void* Realloc(void* Original, size_t, size_t Count)
		{
			if (Count == 0)
			{
				FMemory::Free(Original);
				return nullptr;
			}
			return FMemory::Realloc(Original, Count);
		}

		static void Free(void* Ptr)
		{
			FMemory::Free(Ptr);
		}

		bool operator==(const FAllocatorImpl&) const
		{
			return true;
		}

		bool operator!=(const FAllocatorImpl&) const
		{
			return false;
		}
	};
} // namespace UE::Json::Private

	// types that use TCHAR (common)
	using FEncoding = rapidjson::UTF16<TCHAR>;
	using FAllocator = Private::FAllocatorImpl;
	using FDocument = rapidjson::GenericDocument<FEncoding, FAllocator>;
	using FStringRef = FDocument::StringRefType;
	using FValue = FDocument::ValueType;
	using FMember = FValue::Member;
	
	// types that use TCHAR (for reading)
	using FConstObject = FValue::ConstObject;
	using FConstArray = FValue::ConstArray;

	// types that use TCHAR (for writing)
	using FObject = FValue::Object;
	using FArray = FValue::Array;
	using FStringBuffer = rapidjson::GenericStringBuffer<FEncoding, FAllocator>;
	using FStringWriter = rapidjson::Writer<FStringBuffer, FEncoding, FEncoding, FAllocator>;
	using FPrettyStringWriter = rapidjson::PrettyWriter<FStringBuffer, FEncoding, FEncoding, FAllocator>;

	/**
	 * Make a RapidJSON reference to the given string.
	 * @note The given string must outlive the returned reference.
	 */
	inline FStringRef MakeStringRef(FStringView Str)
	{
		return FStringRef(Str.GetData(), Str.Len());
	}

	/**
	 * Make a RapidJSON value that has a copy of the given string.
	 */
	inline FValue MakeStringValue(FStringView Str, FAllocator& Allocator)
	{
		return FValue(Str.GetData(), Str.Len(), Allocator);
	}

	/**
	 * Reads a string from the specified value.
	 * @note The given value must be a string.
	 */
	inline FStringView ValueAsStringView(const FValue& Value)
	{
		check(Value.IsString());
		return FStringView(Value.GetString(), Value.GetStringLength());
	}

	/**
	 * Attempts to read a string from the specified value.
	 *
	 * Returns the value, if it matches the type explicitly
	 */
	JSON_API TOptional<FStringView> GetStringValue(const FValue& Value);

	/**
	 * Attempts to read a boolean with a specified name.
	 * 
	 * Returns the value, if found and matches the type explicitly
	 */
	JSON_API TOptional<bool> GetBoolField(FConstObject Object, const TCHAR* FieldName);

	/**
	 * Attempts to read an int32 with a specified name.
	 * 
	 * Returns the value, if found and matches the type explicitly
	 */
	JSON_API TOptional<int32> GetInt32Field(FConstObject Object, const TCHAR* FieldName);

	/**
	 * Attempts to read an uint32 with a specified name.
	 * 
	 * Returns the value, if found and matches the type explicitly
	 */
	JSON_API TOptional<uint32> GetUint32Field(FConstObject Object, const TCHAR* FieldName);

	/**
	 * Attempts to read an int64 with a specified name.
	 * 
	 * Returns the value, if found and matches the type explicitly
	 */
	JSON_API TOptional<int64> GetInt64Field(FConstObject Object, const TCHAR* FieldName);

	/**
	 * Attempts to read an uint64 with a specified name.
	 * 
	 * Returns the value, if found and matches the type explicitly
	 */
	JSON_API TOptional<uint64> GetUint64Field(FConstObject Object, const TCHAR* FieldName);

	/**
	 * Attempts to read an double with a specified name.
	 * 
	 * Returns the value, if found and matches the type explicitly
	 */
	JSON_API TOptional<double> GetDoubleField(FConstObject Object, const TCHAR* FieldName);

	/**
	 * Attempts to read a string with a specified name
	 * 
	 * Returns the value, if found and matches the type explicitly
	 */
	JSON_API TOptional<FStringView> GetStringField(FConstObject Object, const TCHAR* FieldName);

	/**
	 * Attempts to parse an enum from a string with a specified name
	 * 
	 * Returns the value, if found and matches the type explicitly
	 */
	template<typename TEnum>
	inline TOptional<TEnum> GetEnumField(FConstObject Object, const TCHAR* FieldName)
	{
		TEnum Value;
		TOptional<FStringView> StringView = GetStringField(Object, FieldName);

		if (StringView.IsSet() && LexTryParseString(Value, *StringView))
		{
			return Value;
		}

		return {};
	}

	/**
	 * Attempts to find an object with the specified name
	 * 
	 * Returns a valid object ref if the field was found and it matches an object.
	 */
	JSON_API TOptional<FConstObject> GetObjectField(FConstObject Object, const TCHAR* FieldName);

	/**
	 * Attempts to find an array with the specified name
	 * 
	 * Returns a valid array ref if the field was found and it matches an array
	 */
	JSON_API TOptional<FConstArray> GetArrayField(FConstObject Object, const TCHAR* FieldName);

	/**
	 * Returns true if a field exists that is a null type
	 */
	JSON_API bool HasNullField(FConstObject Object, const TCHAR* FieldName);

	/**
	 * Returns the root object, if that was the root type for the document
	 */
	JSON_API TOptional<FConstObject> GetRootObject(const FDocument& Document);

	struct FParseError
	{
		rapidjson::ParseErrorCode ErrorCode = rapidjson::kParseErrorNone;
		size_t Offset = 0;

		/**
		 * Create a user-readable error message. Takes the source text parameter to find the line number from the offset
		 */
		JSON_API FString CreateMessage(const FStringView JsonText) const;
	};

	/**
	 * Parse RapidJSON with default flags
	 */
	JSON_API TValueOrError<FDocument, FParseError> Parse(const FStringView JsonText);

	/**
	 * Parse RapidJSON with default flags, into destructible memory, to reduce allocation counts.
	 * 
	 * JsonText must be a zero terminated string per the rapidjson documentation
	 */
	JSON_API TValueOrError<FDocument, FParseError> ParseInPlace(TArrayView<TCHAR> JsonText);

	/**
	 * Write a RapidJSON document as a compact string with default flags.
	 */
	JSON_API FString WriteCompact(const FDocument& Document);

	/**
	 * Write a RapidJSON document as a pretty string with default flags and tab indentation.
	 */
	JSON_API FString WritePretty(const FDocument& Document);

	/**
	 * Return a string representation of type name, for debug logging purposes
	 */
	JSON_API const TCHAR* GetValueTypeName(const FValue& Value);

} // namespace UE::Json
