// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Policies/JsonPrintPolicy.h"

/**
 * Template for print policies that generate compressed output.
 *
 * @param CharType The type of characters to print, i.e. TCHAR or ANSICHAR.
 */
template <class CharType>
struct TCondensedJsonPrintPolicy
	: public TJsonPrintPolicy<CharType>
{
	static inline void WriteLineTerminator(FArchive* Stream) {}
	static inline void WriteTabs(FArchive* Stream, int32 Count) {}
	static inline void WriteSpace(FArchive* Stream) {}
};
