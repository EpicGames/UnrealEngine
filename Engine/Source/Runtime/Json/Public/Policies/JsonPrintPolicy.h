// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Utf8String.h"
#include "CoreMinimal.h"

/**
 * Base template for Json print policies.
 *
 * @param CharType The type of characters to print, i.e. TCHAR or ANSICHAR.
 */
template <class InCharType>
struct TJsonPrintPolicy
{
	using CharType = InCharType;

	/**
	 * Writes a single character to the output stream.
	 *
	 * @param Stream The stream to write to.
	 * @param Char The character to write.
	 */
	static inline void WriteChar( FArchive* Stream, CharType Char )
	{
		Stream->Serialize(&Char, sizeof(CharType));
	}

	/**
	 * Writes a string to the output stream.
	 *
	 * @param Stream The stream to write to.
	 * @param String The string to write.
	 */
	static inline void WriteString(FArchive* Stream, FStringView String)
	{
		auto Conv = StringCast<CharType>(String.GetData(), String.Len());
		Stream->Serialize((void*)Conv.Get(), Conv.Length() * sizeof(CharType));
	}

	/**
	 * Writes a utf8 string to the output stream.
	 *
	 * @param Stream The stream to write to.
	 * @param String The string to write.
	 */
	static inline void WriteString(FArchive* Stream, FUtf8StringView String)
	{
		auto Conv = StringCast<CharType>(String.GetData(), String.Len());
		Stream->Serialize((void*)Conv.Get(), Conv.Length() * sizeof(CharType));
	}

	/**
	 * Writes a float to the output stream.
	 *
	 * @param Stream The stream to write to.
	 * @param Value The float to write.
	 */
	static inline void WriteFloat( FArchive* Stream, float Value )
	{
		WriteString(Stream, FString::Printf(TEXT("%g"), Value));
	}

	/**
	 * Writes a double to the output stream.
	 *
	 * @param Stream The stream to write to.
	 * @param Value The double to write.
	 */
	static inline void WriteDouble(  FArchive* Stream, double Value )
	{
		// Specify 17 significant digits, the most that can ever be useful from a double
		// In particular, this ensures large integers are written correctly
		WriteString(Stream, FString::Printf(TEXT("%.17g"), Value));
	}
};


#if !PLATFORM_TCHAR_IS_CHAR16

/**
 * Specialization for UTF16CHAR that writes FString data UTF-16.
 */
template <>
inline void TJsonPrintPolicy<UTF16CHAR>::WriteString(FArchive* Stream, FStringView String)
{
	// Note: This is a no-op on platforms that are using a 16-bit TCHAR
	FTCHARToUTF16 UTF16String(String.GetData(), String.Len());

	Stream->Serialize((void*)UTF16String.Get(), UTF16String.Length() * sizeof(UTF16CHAR));
}

#endif

