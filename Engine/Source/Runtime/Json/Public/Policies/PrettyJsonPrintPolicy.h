// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Policies/JsonPrintPolicy.h"

/**
 * Template for print policies that generate human readable output.
 *
 * @param CharType The type of characters to print, i.e. TCHAR or ANSICHAR.
 */
template <class CharType>
struct TPrettyJsonPrintPolicy
	: public TJsonPrintPolicy<CharType>
{
	static inline void WriteLineTerminator( FArchive* Stream )
	{
		TJsonPrintPolicy<CharType>::WriteString(Stream, FStringView(LINE_TERMINATOR));
	}

	static inline void WriteTabs( FArchive* Stream, int32 Count )
	{
		CharType Tab = CharType('\t');

		for (int32 i = 0; i < Count; ++i)
		{
			TJsonPrintPolicy<CharType>::WriteChar(Stream, Tab);
		}
	}

	static inline void WriteSpace( FArchive* Stream )
	{
		TJsonPrintPolicy<CharType>::WriteChar(Stream, CharType(' '));
	}
};
