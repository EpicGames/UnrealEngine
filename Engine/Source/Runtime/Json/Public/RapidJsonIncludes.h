// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreMiscDefines.h"

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(push)
#pragma warning(disable : 6282) // Incorrect operator: Assignment of constant in Boolean context. Consider using '==' instead
#pragma warning(disable : 6313) // Incorrect operator: Zero-valued flag cannot be tested with bitwise-and. Use an equality test to check for zero-valued flags
#endif

THIRD_PARTY_INCLUDES_START
#ifndef RAPIDJSON_ASSERT
#define RAPIDJSON_ASSERT(x) check(x)
#endif

#ifndef RAPIDJSON_ERROR_CHARTYPE
#define RAPIDJSON_ERROR_CHARTYPE TCHAR
#endif

#ifndef RAPIDJSON_ERROR_STRING
#define RAPIDJSON_ERROR_STRING(x) TEXT(x)
#endif

// TODO: these cannot be enabled without all includes of rapidjson/document.h being re-directed to RapidJsonIncludes.h
// which will require fixup in uLangJSON

#if 0 
#ifndef RAPIDJSON_MALLOC
#define RAPIDJSON_MALLOC FMemory::Malloc
#endif

#ifndef RAPIDJSON_REALLOC
#define RAPIDJSON_REALLOC FMemory::Realloc
#endif

#ifndef RAPIDJSON_FREE
#define RAPIDJSON_FREE FMemory::Free
#endif
#endif

#if PLATFORM_LITTLE_ENDIAN
#define RAPIDJSON_ENDIAN RAPIDJSON_LITTLEENDIAN
#else
#define RAPIDJSON_ENDIAN RAPIDJSON_BIGENDIAN
#endif

#if PLATFORM_CPU_ARM_FAMILY
#define RAPIDJSON_NEON
#elif PLATFORM_CPU_X86_FAMILY
#define RAPIDJSON_SSE42
#endif

#include "rapidjson/rapidjson.h"
#include "rapidjson/document.h"
#include "rapidjson/encodings.h"
#include "rapidjson/writer.h"
#include "rapidjson/prettywriter.h"
#include "rapidjson/stringbuffer.h"
#include "rapidjson/schema.h"
// just for non-user facing logging purposes use the English error descriptions
#include "rapidjson/error/en.h"
THIRD_PARTY_INCLUDES_END

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(pop)
#endif

