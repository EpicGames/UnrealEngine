// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/JsonSerializable.h"
#include "Misc/Optional.h"

class FJsonSerializerReader;

/**
 * Useful if you just want access to the underlying FJsonObject (for cases where the schema is loose or an outer system will do further de/serialization)
 */
struct FJsonDataBag : public FJsonSerializable
{
	JSON_API virtual void Serialize(FJsonSerializerBase& Serializer, bool bFlatObject) override;

	JSON_API double GetDouble(const FString& Key) const;

	JSON_API FString GetString(const FString& Key) const;

	JSON_API bool GetBool(const FString& Key) const;

	JSON_API TSharedPtr<const FJsonValue> GetField(const FString& Key) const;

	template<typename JSON_TYPE, typename Arg>
	void SetField(const FString& Key, Arg&& Value)
	{
		SetFieldJson(Key, MakeShared<JSON_TYPE>(Forward<Arg>(Value)));
	}

	JSON_API void SetFieldJson(const FString& Key, const TSharedPtr<FJsonValue>& Value);
public:
	TSharedPtr<FJsonObject> JsonObject;

	/* If set, will use TPrettyJsonPrintPolicy */
	TOptional<int32> NumPrintIndents;
};

template<int32 PrintIndentCount>
struct TPrettyJsonBag : public FJsonDataBag
{
	TPrettyJsonBag()
		: FJsonDataBag()
	{
		NumPrintIndents.Emplace(PrintIndentCount);
	}
};
