// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/AnsiString.h"
#include "Containers/Utf8String.h"
#include "Serialization/JsonTypes.h"
#include "Serialization/BufferReader.h"
#include "Misc/StringBuilder.h"

class Error;

#define JSON_NOTATIONMAP_DEF \
static EJsonNotation TokenToNotationTable[] =  \
{ \
	EJsonNotation::Error,			/*EJsonToken::None*/ \
	EJsonNotation::Error,			/*EJsonToken::Comma*/ \
	EJsonNotation::ObjectStart,		/*EJsonToken::CurlyOpen*/ \
	EJsonNotation::ObjectEnd,		/*EJsonToken::CurlyClose*/ \
	EJsonNotation::ArrayStart,		/*EJsonToken::SquareOpen*/ \
	EJsonNotation::ArrayEnd,		/*EJsonToken::SquareClose*/ \
	EJsonNotation::Error,			/*EJsonToken::Colon*/ \
	EJsonNotation::String,			/*EJsonToken::String*/ \
	EJsonNotation::Number,			/*EJsonToken::Number*/ \
	EJsonNotation::Boolean,			/*EJsonToken::True*/ \
	EJsonNotation::Boolean,			/*EJsonToken::False*/ \
	EJsonNotation::Null,			/*EJsonToken::Null*/ \
};

#ifndef WITH_JSON_INLINED_NOTATIONMAP
#define WITH_JSON_INLINED_NOTATIONMAP 0
#endif // WITH_JSON_INLINED_NOTATIONMAP

#if !WITH_JSON_INLINED_NOTATIONMAP
JSON_NOTATIONMAP_DEF;
#endif // WITH_JSON_INLINED_NOTATIONMAP

template <class CharType = TCHAR>
class TJsonReader
{
public:
	// Store ANSICHAR in FUtf8String because JSON may contain scaped unicode characters that would not be recoverable otherwise. Use default string type for the char type otherwise
	using StoredStringType = std::conditional_t<std::is_same_v<CharType, ANSICHAR>, FUtf8String, TString<CharType>>;
	using StoredCharType = TElementType_T<StoredStringType>;
	using ValueAsStringReturnType = std::conditional_t<std::is_same_v<StoredStringType, FString>, const FString&, FString>;

	static TSharedRef< TJsonReader<CharType> > Create( FArchive* const Stream )
	{
		return MakeShareable( new TJsonReader<CharType>( Stream ) );
	}

public:

	virtual ~TJsonReader() = default;

	bool ReadNext( EJsonNotation& Notation )
	{
		if (!ErrorMessage.IsEmpty())
		{
			Notation = EJsonNotation::Error;
			return false;
		}

		if (Stream == nullptr)
		{
			Notation = EJsonNotation::Error;
			SetErrorMessage(TEXT("Null Stream"));
			return true;
		}

		const bool AtEndOfStream = Stream->AtEnd();

		if (AtEndOfStream && !FinishedReadingRootObject)
		{
			Notation = EJsonNotation::Error;
			SetErrorMessage(TEXT("Improperly formatted."));
			return true;
		}

		if (FinishedReadingRootObject && !AtEndOfStream)
		{
			Notation = EJsonNotation::Error;
			SetErrorMessage(TEXT("Unexpected additional input found."));
			return true;
		}

		if (AtEndOfStream)
		{
			return false;
		}

		bool ReadWasSuccess = false;
		Identifier.Empty();

		do
		{
			EJson CurrentState = EJson::None;

			if (ParseState.Num() > 0)
			{
				CurrentState = ParseState.Top();
			}

			switch (CurrentState)
			{
				case EJson::Array:
					ReadWasSuccess = ReadNextArrayValue( /*OUT*/ CurrentToken );
					break;

				case EJson::Object:
					ReadWasSuccess = ReadNextObjectValue( /*OUT*/ CurrentToken );
					break;

				case EJson::None:
				case EJson::Null:
				case EJson::String:
				case EJson::Number:
				case EJson::Boolean:
				default:
					ReadWasSuccess = ReadStart( /*OUT*/ CurrentToken );
					break;
			}
		}
		while (ReadWasSuccess && (CurrentToken == EJsonToken::None));

#if WITH_JSON_INLINED_NOTATIONMAP
		JSON_NOTATIONMAP_DEF;
#endif // WITH_JSON_INLINED_NOTATIONMAP

		Notation = TokenToNotationTable[(int32)CurrentToken];
		FinishedReadingRootObject = ParseState.Num() == 0;

		if (!ReadWasSuccess || (Notation == EJsonNotation::Error))
		{
			Notation = EJsonNotation::Error;

			if (ErrorMessage.IsEmpty())
			{
				SetErrorMessage(TEXT("Unknown Error Occurred"));
			}

			return true;
		}

		if (FinishedReadingRootObject && !Stream->AtEnd())
		{
			ReadWasSuccess = ParseWhiteSpace();
		}

		return ReadWasSuccess;
	}

	bool SkipObject()
	{
		return ReadUntilMatching(EJsonNotation::ObjectEnd);
	}

	bool SkipArray()
	{
		return ReadUntilMatching(EJsonNotation::ArrayEnd);
	}

	inline virtual const FString& GetIdentifier() const { return Identifier; }

	inline virtual ValueAsStringReturnType GetValueAsString() const
	{
		check(CurrentToken == EJsonToken::String);
		if constexpr (std::is_same_v<StoredStringType, FString>)
		{
			return StringValue;
		}
		else
		{
			// Construct FString from a different string type
			return FString(StringValue);
		}
	}

	inline virtual const StoredStringType& GetInternalValueAsString() const
	{
		check(CurrentToken == EJsonToken::String);
		return StringValue;
	}

	inline virtual StoredStringType StealInternalValueAsString()
	{
		check(CurrentToken == EJsonToken::String);
		return MoveTemp(StringValue);
	}

	inline double GetValueAsNumber() const 
	{ 
		check(CurrentToken == EJsonToken::Number);
		return NumberValue;
	}

	inline const StoredStringType& GetValueAsNumberString() const
	{
		check(CurrentToken == EJsonToken::Number);
		return StringValue;
	}
	
	inline bool GetValueAsBoolean() const
	{ 
		check((CurrentToken == EJsonToken::True) || (CurrentToken == EJsonToken::False));
		return BoolValue; 
	}

	inline const FString& GetErrorMessage() const
	{
		return ErrorMessage;
	}

	inline const uint32 GetLineNumber() const
	{
		return LineNumber;
	}

	inline const uint32 GetCharacterNumber() const
	{
		return CharacterNumber;
	}

protected:

	/** Hidden default constructor. */
	TJsonReader()
		: ParseState()
		, CurrentToken( EJsonToken::None )
		, Stream( nullptr )
		, Identifier()
		, ErrorMessage()
		, StringValue()
		, NumberValue( 0.0f )
		, LineNumber( 1 )
		, CharacterNumber( 0 )
		, BoolValue( false )
		, FinishedReadingRootObject( false )
	{ }

	/**
	 * Creates and initializes a new instance with the given input.
	 *
	 * @param InStream An archive containing the input.
	 */
	explicit TJsonReader(FArchive* InStream)
		: ParseState()
		, CurrentToken(EJsonToken::None)
		, Stream(InStream)
		, Identifier()
		, ErrorMessage()
		, StringValue()
		, NumberValue(0.0f)
		, LineNumber(1)
		, CharacterNumber(0)
		, BoolValue(false)
		, FinishedReadingRootObject(false)
	{ }

private:

	void SetErrorMessage( const FString& Message )
	{
		ErrorMessage = Message + FString::Printf(TEXT(" Line: %u Ch: %u"), LineNumber, CharacterNumber);
	}

	bool ReadUntilMatching( const EJsonNotation ExpectedNotation )
	{
		uint32 ScopeCount = 0;
		EJsonNotation Notation;

		while (ReadNext(Notation))
		{
			if ((ScopeCount == 0) && (Notation == ExpectedNotation))
			{
				return true;
			}

			switch (Notation)
			{
			case EJsonNotation::ObjectStart:
			case EJsonNotation::ArrayStart:
				++ScopeCount;
				break;

			case EJsonNotation::ObjectEnd:
			case EJsonNotation::ArrayEnd:
				--ScopeCount;
				break;

			case EJsonNotation::Boolean:
			case EJsonNotation::Null:
			case EJsonNotation::Number:
			case EJsonNotation::String:
				break;

			case EJsonNotation::Error:
				return false;
				break;
			}
		}

		return !Stream->IsError();
	}

	bool ReadStart( EJsonToken& Token )
	{
		if (!ParseWhiteSpace())
		{
			return false;
		}

		Token = EJsonToken::None;

		if (NextToken(Token) == false)
		{
			return false;
		}

		if ((Token != EJsonToken::CurlyOpen) && (Token != EJsonToken::SquareOpen))
		{
			SetErrorMessage(TEXT("Open Curly or Square Brace token expected, but not found."));
			return false;
		}

		return true;
	}

	bool ReadNextObjectValue( EJsonToken& Token )
	{
		const bool bCommaPrepend = Token != EJsonToken::CurlyOpen;
		Token = EJsonToken::None;

		if (NextToken(Token) == false)
		{
			return false;
		}

		if (Token == EJsonToken::CurlyClose)
		{
			return true;
		}
		else
		{
			if (bCommaPrepend)
			{
				if (Token != EJsonToken::Comma)
				{
					SetErrorMessage( TEXT("Comma token expected, but not found.") );
					return false;
				}

				Token = EJsonToken::None;

				if (!NextToken(Token))
				{
					return false;
				}
			}

			if (Token != EJsonToken::String)
			{
				SetErrorMessage( TEXT("String token expected, but not found.") );
				return false;
			}

			// Move value if possible. A conversion will happen if StringValue is not an FString
			Identifier = FString(MoveTemp(StringValue));
			Token = EJsonToken::None;

			if (!NextToken(Token))
			{
				return false;
			}

			if (Token != EJsonToken::Colon)
			{
				SetErrorMessage( TEXT("Colon token expected, but not found.") );
				return false;
			}

			Token = EJsonToken::None;

			if (!NextToken(Token))
			{
				return false;
			}
		}

		return true;
	}

	bool ReadNextArrayValue( EJsonToken& Token )
	{
		const bool bCommaPrepend = Token != EJsonToken::SquareOpen;

		Token = EJsonToken::None;

		if (!NextToken(Token))
		{
			return false;
		}

		if (Token == EJsonToken::SquareClose)
		{
			return true;
		}
		else
		{
			if (bCommaPrepend)
			{
				if (Token != EJsonToken::Comma)
				{
					SetErrorMessage( TEXT("Comma token expected, but not found.") );
					return false;
				}

				Token = EJsonToken::None;

				if (!NextToken(Token))
				{
					return false;
				}
			}
		}

		return true;
	}

	bool NextToken( EJsonToken& OutToken )
	{
		while (!Stream->AtEnd())
		{
			CharType Char;
			if (!Serialize(&Char, sizeof(CharType)))
			{
				return false;
			}
			++CharacterNumber;

			if (Char == CharType('\0'))
			{
				break;
			}

			if (IsLineBreak(Char))
			{
				++LineNumber;
				CharacterNumber = 0;
			}

			if (!IsWhitespace(Char))
			{
				if (IsJsonNumber(Char))
				{
					bool bParseNumberSucceed = ParseNumberToken(Char);
					if (!bParseNumberSucceed && Char != '-') // Could be -NaN, will return false when fail to parse it as -NaN later on
					{
						return false;
					}

					if (bParseNumberSucceed)
					{
						OutToken = EJsonToken::Number;
						return true;
					}
				}

				switch (Char)
				{
				case CharType('{'):
					OutToken = EJsonToken::CurlyOpen; ParseState.Push( EJson::Object );
					return true;

				case CharType('}'):
					{
						OutToken = EJsonToken::CurlyClose;
						if (ParseState.Num())
						{
							ParseState.Pop();
							return true;
						}
						else
						{
							SetErrorMessage(TEXT("Unknown state reached while parsing Json token."));
							return false;
						}
					}

				case CharType('['):
					OutToken = EJsonToken::SquareOpen; ParseState.Push( EJson::Array );
					return true;

				case CharType(']'):
					{
						OutToken = EJsonToken::SquareClose;
						if (ParseState.Num())
						{
							ParseState.Pop();
							return true;
						}
						else
						{
							SetErrorMessage(TEXT("Unknown state reached while parsing Json token."));
							return false;
						}
					}

				case CharType(':'):
					OutToken = EJsonToken::Colon;
					return true;

				case CharType(','):
					OutToken = EJsonToken::Comma;
					return true;

				case CharType('\"'):
					{
						if (!ParseStringToken())
						{
							return false;
						}

						OutToken = EJsonToken::String;
					}
					return true;

				case CharType('t'): case CharType('T'):
				case CharType('f'): case CharType('F'):
				case CharType('i'): case CharType('I'):
				case CharType('n'): case CharType('N'):
				case CharType('-'):
					{
						StoredStringType Test;
						Test += Char;

						while (!Stream->AtEnd())
						{
							if (!Serialize(&Char, sizeof(CharType)))
							{
								return false;
							}

							if (IsAlphaNumber(Char) 
								|| Char == '(' || Char == ')') // Could be "-nan(ind)" depending on the platform and impl of standard library when write
							{
								++CharacterNumber;
								Test += Char;
							}
							else
							{
								// backtrack and break
								Stream->Seek(Stream->Tell() - sizeof(CharType));
								break;
							}
						}

						if (Test == CHARTEXT(CharType, "False"))
						{
							BoolValue = false;
							OutToken = EJsonToken::False;
							return true;
						}

						if (Test == CHARTEXT(CharType, "True"))
						{
							BoolValue = true;
							OutToken = EJsonToken::True;
							return true;
						}

						if (Test == CHARTEXT(CharType, "Null"))
						{
							OutToken = EJsonToken::Null;
							return true;
						}
						
						if (Test.Compare(CHARTEXT(CharType, "Inf"), ESearchCase::IgnoreCase) == 0)
						{
							StringValue = Test;
							NumberValue = std::numeric_limits<double>::infinity();
							OutToken = EJsonToken::Number;
							return true;
						}

						if (Test.Compare(CHARTEXT(CharType, "-Inf"), ESearchCase::IgnoreCase) == 0)
						{
							StringValue = Test;
							NumberValue = -std::numeric_limits<double>::infinity();
							OutToken = EJsonToken::Number;
							return true;
						}

						if (Test.Compare(CHARTEXT(CharType, "NaN"), ESearchCase::IgnoreCase) == 0)
						{
							StringValue = Test;
							NumberValue = std::numeric_limits<double>::quiet_NaN();
							OutToken = EJsonToken::Number;
							return true;
						}

						if (Test.Compare(TEXT("-NaN"), ESearchCase::IgnoreCase) == 0 ||
							Test.Compare(TEXT("-NaN(ind)"), ESearchCase::IgnoreCase) == 0)
						{
							StringValue = Test;
							NumberValue = -std::numeric_limits<double>::quiet_NaN();
							OutToken = EJsonToken::Number;
							return true;
						}

						SetErrorMessage( TEXT("Invalid Json Token. Check that your member names have quotes around them!") );
						return false;
					}

				default: 
					SetErrorMessage( TEXT("Invalid Json Token.") );
					return false;
				}
			}
		}

		SetErrorMessage( TEXT("Invalid Json Token.") );
		return false;
	}

	bool ParseStringToken()
	{
		TStringBuilderWithBuffer<StoredCharType, 512> StringBuffer;
		TStringBuilderWithBuffer<WIDECHAR, 512> UTF16CodePoints;

		// Add escaped surrogate pairs
		auto ConditionallyAddCodePoints = [&StringBuffer, &UTF16CodePoints]() -> void
		{
			if (UTF16CodePoints.Len() > 0)
			{
				// Will convert to StoredCharType encoding if needed
				StringBuffer.Append(UTF16CodePoints);
				UTF16CodePoints.Reset();
			}
		};

		while (true)
		{
			if (Stream->AtEnd())
			{
				SetErrorMessage( TEXT("String Token Abruptly Ended.") );
				return false;
			}

			CharType Char;
			if (!Serialize(&Char, sizeof(CharType)))
			{
				return false;
			}
			++CharacterNumber;

			if (Char == CharType('\"'))
			{
				ConditionallyAddCodePoints();
				break;
			}

			if (Char == CharType('\\'))
			{
				if (!Serialize(&Char, sizeof(CharType)))
				{
					return false;
				}
				++CharacterNumber;

				if (Char != CharType('u'))
				{
					ConditionallyAddCodePoints();
				}

				switch (Char)
				{
				case CharType('\"'): case CharType('\\'): case CharType('/'): StringBuffer.AppendChar(Char); break;
				case CharType('f'): StringBuffer.AppendChar(CharType('\f')); break;
				case CharType('r'): StringBuffer.AppendChar(CharType('\r')); break;
				case CharType('n'): StringBuffer.AppendChar(CharType('\n')); break;
				case CharType('b'): StringBuffer.AppendChar(CharType('\b')); break;
				case CharType('t'): StringBuffer.AppendChar(CharType('\t')); break;
				case CharType('u'):
					// 4 hex digits, like \uAB23, which is a 16 bit number that we would usually see as 0xAB23
					{
						int32 HexNum = 0;

						for (int32 Radix = 3; Radix >= 0; --Radix)
						{
							if (Stream->AtEnd())
							{
								SetErrorMessage( TEXT("String Token Abruptly Ended.") );
								return false;
							}

							if (!Serialize(&Char, sizeof(CharType)))
							{
								return false;
							}
							++CharacterNumber;

							int32 HexDigit = FParse::HexDigit(static_cast<TCHAR>(Char));

							if ((HexDigit == 0) && (Char != CharType('0')))
							{
								SetErrorMessage( TEXT("Invalid Hexadecimal digit parsed.") );
								return false;
							}

							//@TODO: FLOATPRECISION: this is gross
							HexNum += HexDigit * (int32)FMath::Pow(16.f, (float)Radix);
						}

						UTF16CodePoints.AppendChar((UTF16CHAR)HexNum);
					}
					break;

				default:
					SetErrorMessage( TEXT("Bad Json escaped char.") );
					return false;
				}
			}
			else
			{
				ConditionallyAddCodePoints();
				StringBuffer.AppendChar(Char);
			}
		}

		StringValue = StoredStringType(StringBuffer);

		return true;
	}

	bool ParseNumberToken( CharType FirstChar )
	{
		StoredStringType String;
		int32 State = 0;
		bool UseFirstChar = true;
		bool StateError = false;

		while (true)
		{
			if (Stream->AtEnd())
			{
				SetErrorMessage( TEXT("Number Token Abruptly Ended.") );
				return false;
			}

			CharType Char;
			if (UseFirstChar)
			{
				Char = FirstChar;
				UseFirstChar = false;
			}
			else
			{
				if (!Serialize(&Char, sizeof(CharType)))
				{
					return false;
				}
				++CharacterNumber;
			}

			// The following code doesn't actually derive the Json Number: that is handled
			// by the function FPlatformString::Atod below. This code only ensures the Json Number is
			// EXACTLY to specification
			if (IsJsonNumber(Char))
			{
				// ensure number follows Json format before converting
				// This switch statement is derived from a finite state automata
				// derived from the Json spec. A table was not used for simplicity.
				switch (State)
				{
				case 0:
					if (Char == CharType('-')) { State = 1; }
					else if (Char == CharType('0')) { State = 2; }
					else if (IsNonZeroDigit(Char)) { State = 3; }
					else { StateError = true; }
					break;

				case 1:
					if (Char == CharType('0')) { State = 2; }
					else if (IsNonZeroDigit(Char)) { State = 3; }
					else { StateError = true; }
					break;

				case 2:
					if (Char == CharType('.')) { State = 4; }
					else if (Char == CharType('e') || Char == CharType('E')) { State = 5; }
					else { StateError = true; }
					break;

				case 3:
					if (IsDigit(Char)) { State = 3; }
					else if (Char == CharType('.')) { State = 4; }
					else if (Char == CharType('e') || Char == CharType('E')) { State = 5; }
					else { StateError = true; }
					break;

				case 4:
					if (IsDigit(Char)) { State = 6; }
					else { StateError = true; }
					break;

				case 5:
					if (Char == CharType('-') ||Char == CharType('+')) { State = 7; }
					else if (IsDigit(Char)) { State = 8; }
					else { StateError = true; }
					break;

				case 6:
					if (IsDigit(Char)) { State = 6; }
					else if (Char == CharType('e') || Char == CharType('E')) { State = 5; }
					else { StateError = true; }
					break;

				case 7:
					if (IsDigit(Char)) { State = 8; }
					else { StateError = true; }
					break;

				case 8:
					if (IsDigit(Char)) { State = 8; }
					else { StateError = true; }
					break;

				default:
					SetErrorMessage( TEXT("Unknown state reached in Json Number Token.") );
					return false;
				}

				if (StateError)
				{
					break;
				}

				String += Char;
			}
			else
			{
				// backtrack once because we read a non-number character
				Stream->Seek(Stream->Tell() - sizeof(CharType));
				--CharacterNumber;
				// and now the number is fully tokenized
				break;
			}
		}

		// ensure the number has followed valid Json format
		if (!StateError && ((State == 2) || (State == 3) || (State == 6) || (State == 8)))
		{
			StringValue = String;
			NumberValue = FPlatformString::Atod(*String);
			return true;
		}

		if (FirstChar != '-') // Could be -NaN, will set the error message when fail to parse it as -NaN later on
		{
			SetErrorMessage( TEXT("Poorly formed Json Number Token.") );
		}

		return false;
	}

	bool ParseWhiteSpace()
	{
		while (!Stream->AtEnd())
		{
			CharType Char;
			if (!Serialize(&Char, sizeof(CharType)))
			{
				return false;
			}
			++CharacterNumber;

			if (IsLineBreak(Char))
			{
				++LineNumber;
				CharacterNumber = 0;
			}

			if (!IsWhitespace(Char))
			{
				// backtrack and break
				Stream->Seek(Stream->Tell() - sizeof(CharType));
				--CharacterNumber;
				break;
			}
		}
		return true;
	}

	bool IsLineBreak( const CharType& Char )
	{
		return Char == CharType('\n');
	}

	/** Can't use FChar::IsWhitespace because it is TCHAR specific, and it doesn't handle newlines */
	bool IsWhitespace( const CharType& Char )
	{
		return Char == CharType(' ') || Char == CharType('\t') || Char == CharType('\n') || Char == CharType('\r');
	}

	/** Can't use FChar::IsDigit because it is TCHAR specific, and it doesn't handle all the other Json number characters */
	bool IsJsonNumber( const CharType& Char )
	{
		return ((Char >= CharType('0') && Char <= CharType('9')) ||
			Char == CharType('-') || Char == CharType('.') || Char == CharType('+') || Char == CharType('e') || Char == CharType('E'));
	}

	/** Can't use FChar::IsDigit because it is TCHAR specific */
	bool IsDigit( const CharType& Char )
	{
		return (Char >= CharType('0') && Char <= CharType('9'));
	}

	bool IsNonZeroDigit( const CharType& Char )
	{
		return (Char >= CharType('1') && Char <= CharType('9'));
	}

	/** Can't use FChar::IsAlpha because it is TCHAR specific. Also, this only checks A through Z (no underscores or other characters). */
	bool IsAlphaNumber( const CharType& Char )
	{
		return (Char >= CharType('a') && Char <= CharType('z')) || (Char >= CharType('A') && Char <= CharType('Z'));
	}

protected:
	bool Serialize(void* V, int64 Length)
	{
		Stream->Serialize(V, Length);
		if (Stream->IsError())
		{
			SetErrorMessage(TEXT("Stream I/O Error"));
			return false;
		}
		return true;
	}

	template <typename Type>
	UE_DEPRECATED(5.8, "Remove calls to this function because it does nothing.")
	void InlineCombineSurrogates(Type& String)
	{
	}

protected:
	TArray<EJson> ParseState;
	EJsonToken CurrentToken;

	FArchive* Stream;
	FString Identifier;
	FString ErrorMessage;
	StoredStringType StringValue;
	double NumberValue;
	uint32 LineNumber;
	uint32 CharacterNumber;
	bool BoolValue;
	bool FinishedReadingRootObject;
};


template <typename CharType>
class TJsonStringReader
	: public TJsonReader<CharType>
{
public:
	static TSharedRef<TJsonStringReader> Create(const TString<CharType>& JsonString)
	{
		return MakeShareable(new TJsonStringReader(JsonString));
	}

	static TSharedRef<TJsonStringReader> Create(TString<CharType>&& JsonString)
	{
		return MakeShareable(new TJsonStringReader(MoveTemp(JsonString)));
	}

	const TString<CharType>& GetSourceString() const
	{
		return Content;
	}
public:

	virtual ~TJsonStringReader() = default;

protected:

	/**
	 * Parses a string containing Json information.
	 *
	 * @param JsonString The Json string to parse.
	 */
	explicit TJsonStringReader(const TString<CharType>& JsonString)
		: Content(JsonString)
		, Reader(nullptr)
	{
		InitReader();
	}

	/**
	 * Parses a string containing Json information.
	 *
	 * @param JsonString The Json string to parse.
	 */
	explicit TJsonStringReader(TString<CharType>&& JsonString)
		: Content(MoveTemp(JsonString))
		, Reader(nullptr)
	{
		InitReader();
	}

	inline void InitReader()
	{
		if (Content.IsEmpty())
		{
			return;
		}

		Reader = MakeUnique<FBufferReader>((void*)*Content, Content.Len() * sizeof(CharType), false);
		this->Stream = Reader.Get();
	}

protected:
	const TString<CharType> Content;
	TUniquePtr<FBufferReader> Reader;
};

using FJsonStringReader = TJsonStringReader<TCHAR>;

template <class CharType>
class TJsonStringViewReader
	: public TJsonReader<CharType>
{
public:

	static TSharedRef<TJsonStringViewReader> Create(TStringView<CharType> JsonString)
	{
		return MakeShareable(new TJsonStringViewReader(JsonString));
	}

public:

	virtual ~TJsonStringViewReader() = default;

protected:

	/**
	 * Parses a string containing Json information.
	 *
	 * @param JsonString The Json string to parse.
	 */
	explicit TJsonStringViewReader(TStringView<CharType> JsonString)
		: Content(JsonString)
		, Reader(nullptr)
	{
		InitReader();
	}

	void InitReader()
	{
		if (Content.IsEmpty())
		{
			return;
		}

		Reader = MakeUnique<FBufferReader>((void*)Content.GetData(), Content.Len() * sizeof(CharType), false);
		TJsonReader<CharType>::Stream = Reader.Get();
	}

protected:
	TStringView<CharType> Content;
	TUniquePtr<FBufferReader> Reader;
};

template <class CharType = TCHAR>
class TJsonReaderFactory
{
public:
	using StringType = TString<CharType>;

	static TSharedRef<TJsonReader<TElementType_T<StringType>>> Create(StringType&& JsonString)
	{
		return TJsonStringReader<TElementType_T<StringType>>::Create(MoveTemp(JsonString));
	}

	static TSharedRef<TJsonReader<TElementType_T<StringType>>> Create(const StringType& JsonString)
	{
		return TJsonStringReader<TElementType_T<StringType>>::Create(JsonString);
	}

	static TSharedRef<TJsonReader<CharType>> Create(FArchive* const Stream)
	{
		return TJsonReader<CharType>::Create(Stream);
	}

	static TSharedRef<TJsonReader<CharType>> CreateFromView(TStringView<CharType> JsonString)
	{
		return TJsonStringViewReader<CharType>::Create(JsonString);
	}
};
