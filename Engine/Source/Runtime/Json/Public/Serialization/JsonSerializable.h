// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dom/JsonObject.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonSerializerReader.h"
#include "JsonSerializerWriter.h"
#include "Misc/TVariant.h"
#include "Templates/Overload.h"

/**
 * Base class for a JSON serializable object
 */
struct FJsonSerializable
{
	/**
	 *	Virtualize destructor as we provide overridable functions
	 */
	JSON_API virtual ~FJsonSerializable();

	/**
	 * Used to allow serialization of a const ref
	 *
	 * @return the corresponding json string
	 */
	JSON_API const FString ToJson(bool bPrettyPrint = true) const;
	JSON_API const FUtf8String ToJsonUtf8(bool bPrettyPrint = true) const;
	
	/**
	 * Serializes this object to its JSON string form
	 *
	 * @param bPrettyPrint - If true, will use the pretty json formatter
	 * @return the corresponding json string
	 */
	JSON_API virtual const FString ToJson(bool bPrettyPrint=true);
	JSON_API virtual const FUtf8String ToJsonUtf8(bool bPrettyPrint = true);
	
		/**
	 * Serializes this object with a Json Writer
	 * 
	 * @param JsonWriter - The writer to use
	 * @param bFlatObject if true then no object wrapper is used
	 */
	template<class CharType, class PrintPolicy, ESPMode SPMode>
	void ToJson(TSharedRef<TJsonWriter<CharType, PrintPolicy>, SPMode> JsonWriter, bool bFlatObject = false) const;

	/**
	 * Serializes the contents of a JSON string into this object
	 *
	 * @param Json the JSON data to serialize from
	 */
	JSON_API virtual bool FromJson(const TCHAR* Json);
	JSON_API virtual bool FromJson(const UTF8CHAR* Json);
	JSON_API virtual bool FromJson(const FString& Json);
	JSON_API virtual bool FromJson(const FUtf8String& Json);

	/**
	 * Serializes the contents of a JSON string into this object
	 *
	 * @param Json the JSON data to serialize from
	 */
	JSON_API virtual bool FromJson(FString&& Json);
	JSON_API virtual bool FromJson(FUtf8String&& Json);

	/**
	 * Serializes the contents of a JSON string into this object using FUtf8StringView
	 *
	 * @param JsonStringView the JSON data to serialize from
	 */
	JSON_API bool FromJsonStringView(FUtf8StringView JsonStringView);

	/**
	 * Serializes the contents of a JSON string into this object using FWideStringView
	 *
	 * @param JsonStringView the JSON data to serialize from
	 */
	JSON_API bool FromJsonStringView(FWideStringView JsonStringView);

	JSON_API virtual bool FromJson(TSharedPtr<FJsonObject> JsonObject);

	/**
	 * Abstract method that needs to be supplied using the macros
	 *
	 * @param Serializer the object that will perform serialization in/out of JSON
	 * @param bFlatObject if true then no object wrapper is used
	 */
	virtual void Serialize(FJsonSerializerBase& Serializer, bool bFlatObject) = 0;
};

template<class CharType, class PrintPolicy, ESPMode SPMode>
inline void FJsonSerializable::ToJson(TSharedRef<TJsonWriter<CharType, PrintPolicy>, SPMode> JsonWriter, bool bFlatObject) const
{
	FJsonSerializerWriter<CharType, PrintPolicy> Serializer(MoveTemp(JsonWriter));
	const_cast<FJsonSerializable*>(this)->Serialize(Serializer, bFlatObject);
}

namespace UE::JsonArray
{

namespace Private
{

template<typename T, typename CharType>
inline bool FromJson(TArray<T>& OutArray, TStringView<CharType> JsonString)
{
	OutArray.Reset();

	TArray<TSharedPtr<FJsonValue>> ArrayValues;
	TSharedRef<TJsonReader<CharType>> JsonReader = TJsonReaderFactory<CharType>::CreateFromView(JsonString);
	if (FJsonSerializer::Deserialize(JsonReader, ArrayValues))
	{
		for (const TSharedPtr<FJsonValue>& Value : ArrayValues)
		{
			TSharedPtr<FJsonObject>* ArrayEntry;
			if (Value.IsValid() && Value->TryGetObject(ArrayEntry))
			{
				if (ArrayEntry && ArrayEntry->IsValid())
				{
					FJsonSerializerReader Serializer(*ArrayEntry);
					OutArray.Add_GetRef(T()).Serialize(Serializer, false);
				}
				else
				{
					UE_LOGF(LogJson, Error, "Failed to parse Json from array");
					return false;
				}
			}
		}

		return true;
	}

	return false;
}

using FReturnStringArgs = TTuple<FString* /*OutValue*/, bool /*bPrettyPrint*/>;

using FPrettyWriter = TSharedRef<TJsonWriter<>>;
using FCondensedWriter = TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>>;
using FWriterVariants = TVariant<FPrettyWriter, FCondensedWriter>;

using FToJsonVariantArgs = TVariant<FReturnStringArgs, FWriterVariants>;

using FPrettySerializer = FJsonSerializerWriter<>;
using FCondensedSerializer = FJsonSerializerWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>;

template<typename T, typename...SerializerArgsT>
inline void ToJson_SerializeArrayElements(TArray<T>& InArray, SerializerArgsT...Args)
{
	for (T& ArrayEntry : InArray)
	{
		ArrayEntry.Serialize(Args...);
	}
}

template<typename T, typename...SerializerArgsT>
inline void ToJson_SerializeArrayElements(TArray<T*>& InArray, SerializerArgsT...Args)
{
	for (T* ArrayEntry : InArray)
	{
		ArrayEntry->Serialize(Args...);
	}
}

template<typename T>
inline void ToJson(TArray<T>& InArray, const FToJsonVariantArgs& InArgs)
{
	using FPrettySerializerAndWriter = TTuple<FPrettySerializer, FPrettyWriter>;
	using FCondensedSerializerAndWriter = TTuple<FCondensedSerializer, FCondensedWriter>;

	using FSerializerVariant = TVariant<FPrettySerializerAndWriter, FCondensedSerializerAndWriter>;

	FSerializerVariant SerializerToUse = ::Visit(UE::Overload(
		[](const FReturnStringArgs& ReturnStringArgs)
		{
			if (ReturnStringArgs.template Get<1>())
			{
				FPrettyWriter NewWriter = TJsonWriterFactory<>::Create(ReturnStringArgs.template Get<0>());
				return FSerializerVariant(TInPlaceType<FPrettySerializerAndWriter>(), FPrettySerializer(NewWriter), NewWriter);
			}
			else
			{
				FCondensedWriter NewWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>::Create(ReturnStringArgs.template Get<0>());
				return FSerializerVariant(TInPlaceType<FCondensedSerializerAndWriter>(), FCondensedSerializer(NewWriter), NewWriter);
			}
		},
		[](const FWriterVariants& WriterVariants)
		{
			return ::Visit(UE::Overload(
				[](const FPrettyWriter& PrettyWriter)
				{
					return FSerializerVariant(TInPlaceType<FPrettySerializerAndWriter>(), FPrettySerializer(PrettyWriter), PrettyWriter);
				},
				[](const FCondensedWriter& CondensedWriter)
				{
					return FSerializerVariant(TInPlaceType<FCondensedSerializerAndWriter>(), FCondensedSerializer(CondensedWriter), CondensedWriter);
				}
			), WriterVariants);
		}
	), InArgs);

	const bool bCloseWriter = InArgs.IsType<FReturnStringArgs>();

	::Visit([bCloseWriter, &InArray](auto& StoredSerializer)
		{
			StoredSerializer.template Get<0>().StartArray();

			ToJson_SerializeArrayElements(InArray, StoredSerializer.template Get<0>(), false);

			StoredSerializer.template Get<0>().EndArray();

			if (bCloseWriter)
			{
				StoredSerializer.template Get<1>()->Close();
			}
		}, SerializerToUse);
}
} // namespace Private

template<typename T>
static bool FromJson(TArray<T>& OutArray, const FString& JsonString)
{
	return Private::FromJson(OutArray, FStringView(JsonString));
}

template<typename T>
static bool FromJson(TArray<T>& OutArray, FString&& JsonString)
{
	return Private::FromJson(OutArray, FStringView(MoveTemp(JsonString)));
}

template<typename T>
static bool FromJson(TArray<T>& OutArray, FUtf8StringView JsonStringView)
{
	return Private::FromJson(OutArray, JsonStringView);
}

template<typename T>
static bool FromJson(TArray<T>& OutArray, FWideStringView JsonStringView)
{
	return Private::FromJson(OutArray, JsonStringView);
}

/* non-const due to T::Serialize being a non-const function */
template<typename T>
static const FString ToJson(TArray<T>& InArray, const bool bPrettyPrint = true)
{
	FString JsonStr;
	Private::ToJson(InArray, Private::FToJsonVariantArgs(TInPlaceType<Private::FReturnStringArgs>(), &JsonStr, bPrettyPrint));
	return JsonStr;
}

template<typename T>
static void ToJson(TArray<T>& InArray, Private::FPrettyWriter& JsonWriter)
{
	Private::ToJson(InArray, Private::FToJsonVariantArgs(TInPlaceType<Private::FWriterVariants>(), Private::FWriterVariants(TInPlaceType<Private::FPrettyWriter>(), JsonWriter)));
}

template<typename T>
static void ToJson(TArray<T>& InArray, Private::FCondensedWriter& JsonWriter)
{
	Private::ToJson(InArray, Private::FToJsonVariantArgs(TInPlaceType<Private::FWriterVariants>(), Private::FWriterVariants(TInPlaceType<Private::FCondensedWriter>(), JsonWriter)));
}

} // namespace UE::JsonArray
