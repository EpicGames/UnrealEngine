// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonTypes.h"
#include "Serialization/JsonWriter.h"

class Error;

struct FJsonSerializerPolicy_JsonObject
{
	using FValue = TSharedPtr<FJsonValue>;
	using FArrayOfValues = TArray<TSharedPtr<FJsonValue>>;
	using FMapOfValues = TSharedPtr<FJsonObject>;

	struct FGlobalState
	{
#if !UE_JSONOBJECT_LEGACY_STRING_KEYS
		TSharedPtr<UE::JSON::Private::FJsonStringSet> StringSet = MakeShared<UE::JSON::Private::FJsonStringSet>();
#endif
	};

	struct FStackState
	{
		FStackState() = default;
		explicit FStackState(const FGlobalState& GlobalState)
#if !UE_JSONOBJECT_LEGACY_STRING_KEYS
			: StringSet(GlobalState.StringSet)
#endif
		{
		}

		EJson Type;
		FString Identifier;
		TArray<TSharedPtr<FJsonValue>> Array;
		TSharedPtr<FJsonObject> Object;
#if !UE_JSONOBJECT_LEGACY_STRING_KEYS
		TSharedPtr<UE::JSON::Private::FJsonStringSet> StringSet;
#endif
	};

	struct FElement
	{
		FElement( const FValue& InValue )
			: Identifier()
			, Value(InValue)
		{ }

		FElement( const FMapOfValues& Object )
			: Identifier()
			, Value(MakeShared<FJsonValueObject>(Object))
		{ }

		FElement( const FArrayOfValues& Array )
			: Identifier()
			, Value(MakeShared<FJsonValueArray>(Array))
		{ }

		FElement( const FString& InIdentifier, const FValue& InValue )
			: Identifier( InIdentifier )
			, Value( InValue )
			, bIsKeyValuePair( true )
		{ }

		FString Identifier;
		FValue Value;
		bool bHasBeenProcessed = false;
		bool bIsKeyValuePair = false;
	};

	JSON_API static bool GetValueFromState(const FStackState& State, FValue& OutValue);
	JSON_API static bool GetValueFromState(const FStackState& State, FArrayOfValues& OutArray);
	JSON_API static bool GetValueFromState(const FStackState& State, FMapOfValues& OutMap);
	JSON_API static void ResetValue(FValue& OutValue);
	JSON_API static void ReadObjectStart(FStackState& State);
	JSON_API static void ReadObjectEnd(FStackState& State, FValue& OutValue);
	JSON_API static void ReadArrayStart(FStackState& State);
	JSON_API static void ReadArrayEnd(FStackState& State, FValue& OutValue);

	template<class CharType>
	static void ReadBoolean(TJsonReader<CharType>& Reader, FValue& OutValue)
	{
		OutValue = MakeShared<FJsonValueBoolean>(Reader.GetValueAsBoolean());
	}

	template<class CharType>
	static void ReadString(TJsonReader<CharType>& Reader, FValue& OutValue)
	{
		using StoredCharType = typename TJsonReader<CharType>::StoredCharType;
		OutValue = MakeShared<TJsonValueString<StoredCharType>>(Reader.StealInternalValueAsString());
	}

	template<class CharType>
	static void ReadNumberAsString(TJsonReader<CharType>& Reader, FValue& OutValue)
	{
		using StoredCharType = typename TJsonReader<CharType>::StoredCharType;
		OutValue = MakeShared<TJsonValueNumberString<StoredCharType>>(Reader.GetValueAsNumberString());
	}

	template<class CharType>
	static void ReadNumber(TJsonReader<CharType>& Reader, FValue& OutValue)
	{
		OutValue = MakeShared<FJsonValueNumber>(Reader.GetValueAsNumber());
	}

	JSON_API static void ReadNull(FValue& OutValue);
	JSON_API static void AddValueToObject(FStackState& State, const FString& Identifier, FValue& NewValue);
	JSON_API static void AddValueToArray(FStackState& State, FValue& NewValue);

	template<class CharType, class PrintPolicy>
	static bool SerializeIfBool(TSharedRef<FElement>& Element,  TJsonWriter<CharType, PrintPolicy>& Writer, bool bWriteValueOnly)
	{
		if (Element->Value->Type == EJson::Boolean)
		{
			if (bWriteValueOnly)
			{
				Writer.WriteValue(Element->Value->AsBool());
			}
			else
			{
				Writer.WriteValue(Element->Identifier, Element->Value->AsBool());
			}

			return true;
		}

		return false;
	}

	template<class CharType, class PrintPolicy>
	static bool SerializeIfNumber(TSharedRef<FElement>& Element,  TJsonWriter<CharType, PrintPolicy>& Writer, bool bWriteValueOnly)
	{
		if (Element->Value->Type == EJson::Number)
		{
			if (bWriteValueOnly)
			{
				if (Element->Value->PreferStringRepresentation())
				{
					Writer.WriteRawJSONValue(Element->Value->AsString());
				}
				else
				{
					Writer.WriteValue(Element->Value->AsNumber());
				}
			}
			else
			{
				if (Element->Value->PreferStringRepresentation())
				{
					Writer.WriteRawJSONValue(Element->Identifier, Element->Value->AsString());
				}
				else
				{
					Writer.WriteValue(Element->Identifier, Element->Value->AsNumber());
				}
			}

			return true;
		}

		return false;
	}

	template<class CharType, class PrintPolicy>
	static bool SerializeIfString(TSharedRef<FElement>& Element,  TJsonWriter<CharType, PrintPolicy>& Writer, bool bWriteValueOnly)
	{
		if(Element->Value->Type == EJson::String)
		{
			if (bWriteValueOnly)
			{
				Writer.WriteValue(Element->Value->AsString());
			}
			else
			{
				Writer.WriteValue(Element->Identifier, Element->Value->AsString());
			}

			return true;
		}

		return false;
	}

	template<class CharType, class PrintPolicy>
	static bool SerializeIfNull(TSharedRef<FElement>& Element,  TJsonWriter<CharType, PrintPolicy>& Writer, bool bWriteValueOnly)
	{
		if(Element->Value->Type == EJson::Null)
		{
			if (bWriteValueOnly)
			{
				Writer.WriteNull();
			}
			else
			{
				Writer.WriteNull(Element->Identifier);
			}
			
			return true;
		}
		
		return false;
	}

	template<class CharType, class PrintPolicy>
	static bool SerializeIfArray(TArray<TSharedRef<FElement>>& ElementStack, TSharedRef<FElement>& Element,  TJsonWriter<CharType, PrintPolicy>& Writer, bool bWriteValueOnly)
	{
		if(Element->Value->Type == EJson::Array)
		{
			if (Element->bHasBeenProcessed)
			{
				Writer.WriteArrayEnd();
			}
			else
			{
				Element->bHasBeenProcessed = true;
				ElementStack.Push(Element);

				if (bWriteValueOnly)
				{
					Writer.WriteArrayStart();
				}
				else
				{
					Writer.WriteArrayStart(Element->Identifier);
				}

				FArrayOfValues Values = Element->Value->AsArray();

				for (int Index = Values.Num() - 1; Index >= 0; --Index)
				{
					ElementStack.Push(MakeShared<FElement>(Values[Index]));
				}
			}

			return true;
		}
		
		return false;
	}

	template<class CharType, class PrintPolicy>
	static bool SerializeIfObject(TArray<TSharedRef<FElement>>& ElementStack, TSharedRef<FElement>& Element,  TJsonWriter<CharType, PrintPolicy>& Writer, bool bWriteValueOnly)
	{
		if(Element->Value->Type == EJson::Object)
		{
			if (Element->bHasBeenProcessed)
			{
				Writer.WriteObjectEnd();
			}
			else
			{
				Element->bHasBeenProcessed = true;
				ElementStack.Push(Element);

				if (bWriteValueOnly)
				{
					Writer.WriteObjectStart();
				}
				else
				{
					Writer.WriteObjectStart(Element->Identifier);
				}

				TArray<FJsonObject::FStringType> Keys;
				FArrayOfValues Values;
				FMapOfValues ElementObject = Element->Value->AsObject();
				ElementObject->Values.GenerateKeyArray(Keys);
				ElementObject->Values.GenerateValueArray(Values);

				check(Keys.Num() == Values.Num());

				for (int Index = Values.Num() - 1; Index >= 0; --Index)
				{
					ElementStack.Push(MakeShared<FElement>(*Keys[Index], Values[Index]));
				}
			}

			return true;
		}
		
		return false;
	}
};

template<typename Policy = FJsonSerializerPolicy_JsonObject>
class TJsonSerializer
{
public:

	enum class EFlags
	{
		None = 0,
		StoreNumbersAsStrings = 1,
	};

	template <class CharType>
	static bool Deserialize(const TSharedRef<TJsonReader<CharType>>& Reader, typename Policy::FValue& OutValue, EFlags InOptions = EFlags::None)
	{
		return Deserialize(*Reader, OutValue, InOptions);
	}

	template <class CharType>
	static bool Deserialize(TJsonReader<CharType>& Reader, typename Policy::FValue& OutValue, EFlags InOptions = EFlags::None)
	{
		typename Policy::FStackState State;
		if (!Deserialize(Reader, /*OUT*/State, InOptions))
		{
			return false;
		}

		return Policy::GetValueFromState(State, OutValue);
	}

	template <class CharType>
	static bool Deserialize(const TSharedRef<TJsonReader<CharType>>& Reader, typename Policy::FArrayOfValues& OutArray, EFlags InOptions = EFlags::None)
	{
		return Deserialize(*Reader, OutArray, InOptions);
	}

	template <class CharType>
	static bool Deserialize(TJsonReader<CharType>& Reader, typename Policy::FArrayOfValues& OutArray, EFlags InOptions = EFlags::None)
	{
		typename Policy::FStackState State;
		if (!Deserialize(Reader, /*OUT*/State, InOptions))
		{
			return false;
		}

		return Policy::GetValueFromState(State, OutArray);
	}

	template <class CharType>
	static bool Deserialize(const TSharedRef<TJsonReader<CharType>>& Reader, typename Policy::FMapOfValues& OutMap, EFlags InOptions = EFlags::None)
	{
		return Deserialize(*Reader, OutMap, InOptions);
	}

	template <class CharType>
	static bool Deserialize(TJsonReader<CharType>& Reader, typename Policy::FMapOfValues& OutMap, EFlags InOptions = EFlags::None)
	{
		typename Policy::FStackState State;
		if (!Deserialize(Reader, /*OUT*/State, InOptions))
		{
			return false;
		}

		return Policy::GetValueFromState(State, OutMap);
	}

	/**
	 * Serialize the passed value and identifier into the writer.
	 * Empty string identifiers will be ignored when the writer is not writing inside of a map of values and only the value will be serialized.
	 * If the writer is in a state where it's currently writing inside of a map of values, then the identifier will always be serialized.
	 * 
	 * Json Examples:
	 *	- Writer state: { "foo": "bar" <writer position>
	 *    Parameters: Identifier: ""
	 *                Value: "baz"
	 *    Serialization result: { "foo": "bar", "": "baz" <writer position> //empty identifier is serialized as a valid key for the key:value pair "":"baz"
	 *
	 * - Writer state: { "foo": ["bar" <writer position>
	 *   Parameters: Identifier: ""
	 *               Value: "baz"
	 *   Serialization result: { foo: ["bar", "baz" <writer position> //empty identifier is ignored since we are writing into an array and not an object.
	 *
	 * @param Value			The value we are serializing
	 * @param Identifier	The identifier of the value, empty identifiers are ignored outside of maps of values.
	 * @param Writer		The writer the value and identifier are written into.
	 * @param bCloseWriter	When set to true the Writer will be closed after the serialization.
	 * @return				Returns true if the serialization was successful, false otherwise.
	 */
	template <class CharType, class PrintPolicy>
	static bool Serialize(const typename Policy::FValue& Value, const FString& Identifier, const TSharedRef<TJsonWriter<CharType, PrintPolicy>>& Writer, bool bCloseWriter = true)
	{
		return Serialize(Value, Identifier, *Writer, bCloseWriter);
	}

	/**
	 * Serialize the passed value and identifier into the writer.
	 * Empty string identifiers will be ignored when the writer is not writing inside of a map of values and only the value will be serialized.
	 * If the writer is in a state where it's currently writing inside of a map of values, then the identifier will always be serialized.
	 * 
	 * Json Examples:
	 *	- Writer state: { "foo": "bar" <writer position>
	 *    Parameters: Identifier: ""
	 *                Value: "baz"
	 *    Serialization result: { "foo": "bar", "": "baz" <writer position> //empty identifier is serialized as a valid key for the key:value pair "":"baz"
	 *
	 * - Writer state: { "foo": ["bar" <writer position>
	 *   Parameters: Identifier: ""
	 *               Value: "baz"
	 *   Serialization result: { foo: ["bar", "baz" <writer position> //empty identifier is ignored since we are writing into an array and not an object.
	 *
	 * @param Value			The value we are serializing
	 * @param Identifier	The identifier of the value, empty identifiers are ignored outside of maps of values.
	 * @param Writer		The writer the value and identifier are written into.
	 * @param bCloseWriter	When set to true the Writer will be closed after the serialization.
	 * @return				Returns true if the serialization was successful, false otherwise.
	 */
	template <class CharType, class PrintPolicy>
	static bool Serialize(const typename Policy::FValue& Value, const FString& Identifier, TJsonWriter<CharType, PrintPolicy>& Writer, bool bCloseWriter = true)
	{
		const TSharedRef<typename Policy::FElement> StartingElement = MakeShared<typename Policy::FElement>(Identifier, Value);
		return Serialize<CharType, PrintPolicy>(StartingElement, Writer, bCloseWriter);
	}

	/**
	 * Serialize the passed array of values into the writer.
	 * This will effectively serialize all of the values enclosed in [] square brackets.
	 * 
	 * Json Example:
	 *  - Writer state: [123 <writer position>
	 *    Parameter: Array: ["foo", "bar", "", 456]
	 *    Serialization result: [123, ["foo", "bar", "", 456] <writer position>
	 *
	 * @param Array		    The array we are serializing
	 * @param Writer		The writer the array is written into.
	 * @param bCloseWriter	When set to true the Writer will be closed after the serialization.
	 * @return				Returns true if the serialization was successful, false otherwise.
	 */
	template <class CharType, class PrintPolicy>
	static bool Serialize(const typename Policy::FArrayOfValues& Array, const TSharedRef<TJsonWriter<CharType, PrintPolicy>>& Writer, bool bCloseWriter = true)
	{
		return Serialize(Array, *Writer, bCloseWriter);
	}

	/**
	 * Serialize the passed array of values into the writer.
	 * This will effectively serialize all of the values enclosed in [] square brackets.
	 * 
	 * Json Example:
	 *  - Writer state: [123 <writer position>
	 *    Parameter: Array: ["foo", "bar", "", 456]
	 *    Serialization result: [123, ["foo", "bar", "", 456] <writer position>
	 *
	 * @param Array		    The array we are serializing
	 * @param Writer		The writer the array is written into.
	 * @param bCloseWriter	When set to true the Writer will be closed after the serialization.
	 * @return				Returns true if the serialization was successful, false otherwise.
	 */
	template <class CharType, class PrintPolicy>
	static bool Serialize(const typename Policy::FArrayOfValues& Array, TJsonWriter<CharType, PrintPolicy>& Writer, bool bCloseWriter = true )
	{
		const TSharedRef<typename Policy::FElement> StartingElement = MakeShared<typename Policy::FElement>(Array);
		return Serialize<CharType, PrintPolicy>(StartingElement, Writer, bCloseWriter);
	}

	/**
	 * Serialize the passed map of values into the writer.
	 * This will effectively serialize all of the identifier:value pairs of the map, enclosed in {} curly brackets.
	 * 
	 * Json Example:
	 *  - Writer state: [123 <writer position>
	 *    Parameter: Object: {"foo": "bar", "baz": "", "": 456}
	 *    Serialization result: [123, {"foo": "bar", "baz": "", "": 456} <writer position>
	 *
	 * @param Object		The map of values we are serializing
	 * @param Writer		The writer the object is written into.
	 * @param bCloseWriter	When set to true the Writer will be closed after the serialization.
	 * @return				Returns true if the serialization was successful, false otherwise.
	 */
	template <class CharType, class PrintPolicy>
	static bool Serialize(const typename Policy::FMapOfValues& Object, const TSharedRef<TJsonWriter<CharType, PrintPolicy>>& Writer, bool bCloseWriter = true )
	{
		return Serialize(Object, *Writer, bCloseWriter);
	}

	/**
	 * Serialize the passed map of values into the writer.
	 * This will effectively serialize all of the identifier:value pairs of the map, enclosed in {} curly brackets.
	 * 
	 * Json Example:
	 *  - Writer state: [123 <writer position>
	 *    Parameter: Object: {"foo": "bar", "baz": "", "": 456}
	 *    Serialization result: [123, {"foo": "bar", "baz": "", "": 456} <writer position>
	 *
	 * @param Object		The map of values we are serializing
	 * @param Writer		The writer the object is written into.
	 * @param bCloseWriter	When set to true the Writer will be closed after the serialization.
	 * @return				Returns true if the serialization was successful, false otherwise.
	 */
	template <class CharType, class PrintPolicy>
	static bool Serialize(const typename Policy::FMapOfValues& Object, TJsonWriter<CharType, PrintPolicy>& Writer, bool bCloseWriter = true)
	{
		const TSharedRef<typename Policy::FElement> StartingElement = MakeShared<typename Policy::FElement>(Object);
		return Serialize<CharType, PrintPolicy>(StartingElement, Writer, bCloseWriter);
	}

private:
	template <class CharType>
	static bool Deserialize(TJsonReader<CharType>& Reader, typename Policy::FStackState& OutStackState, EFlags InOptions)
	{
		typename Policy::FGlobalState GlobalState;
		TArray<TSharedRef<typename Policy::FStackState>> ScopeStack; 
		TSharedPtr<typename Policy::FStackState> CurrentState;

		typename Policy::FValue NewValue;
		EJsonNotation Notation;

		while (Reader.ReadNext(Notation))
		{
			FString Identifier = Reader.GetIdentifier();

			Policy::ResetValue(NewValue);

			bool bWasValueRead = false;

			switch( Notation )
			{
			case EJsonNotation::ObjectStart:
				{
					if (CurrentState.IsValid())
					{
						ScopeStack.Push(CurrentState.ToSharedRef());
					}

					CurrentState = MakeShared<typename Policy::FStackState>(GlobalState);
					CurrentState->Identifier = Identifier;
					Policy::ReadObjectStart(*CurrentState);
				}
				break;

			case EJsonNotation::ObjectEnd:
				{
					if (ScopeStack.Num() > 0)
					{
						Identifier = CurrentState->Identifier;
						Policy::ReadObjectEnd(*CurrentState, NewValue);
						bWasValueRead = true;
						CurrentState = ScopeStack.Pop();
					}
				}
				break;

			case EJsonNotation::ArrayStart:
				{
					if (CurrentState.IsValid())
					{
						ScopeStack.Push(CurrentState.ToSharedRef());
					}

					CurrentState = MakeShared<typename Policy::FStackState>(GlobalState);
					Policy::ReadArrayStart(*CurrentState);
					CurrentState->Identifier = Identifier;
				}
				break;

			case EJsonNotation::ArrayEnd:
				{
					if (ScopeStack.Num() > 0)
					{
						Identifier = CurrentState->Identifier;
						Policy::ReadArrayEnd(*CurrentState, NewValue);
						bWasValueRead = true;
						CurrentState = ScopeStack.Pop();
					}
				}
				break;

			case EJsonNotation::Boolean:
				Policy::ReadBoolean(Reader, NewValue); bWasValueRead = true;
				break;

			case EJsonNotation::String:
				Policy::ReadString(Reader, NewValue); bWasValueRead = true;
				break;

			case EJsonNotation::Number:
				{
					if (EnumHasAnyFlags(InOptions, EFlags::StoreNumbersAsStrings))
					{
						Policy::ReadNumberAsString(Reader, NewValue);
					}
					else
					{
						Policy::ReadNumber(Reader, NewValue);
					}

					bWasValueRead = true;
				}
				break;

			case EJsonNotation::Null:
				Policy::ReadNull(NewValue); bWasValueRead = true;
				break;

			case EJsonNotation::Error:
				return false;
				break;
			}

			if (bWasValueRead && CurrentState.IsValid())
			{
				if (CurrentState->Type == EJson::Object)
				{
					Policy::AddValueToObject(*CurrentState, Identifier, NewValue);
				}
				else
				{
					Policy::AddValueToArray(*CurrentState, NewValue);
				}
			}
		}

		if (!CurrentState.IsValid() || !Reader.GetErrorMessage().IsEmpty())
		{
			return false;
		}

		OutStackState = *CurrentState.Get();

		return true;
	}	

	template <class CharType, class PrintPolicy>
	static bool Serialize(const TSharedRef<typename Policy::FElement>& StartingElement, TJsonWriter<CharType, PrintPolicy>& Writer, bool bCloseWriter)
	{
		TArray<TSharedRef<typename Policy::FElement>> ElementStack;
		ElementStack.Push(StartingElement);

		while (ElementStack.Num() > 0)
		{
			TSharedRef<typename Policy::FElement> Element = ElementStack.Pop();			

			// Empty keys are valid identifiers only when writing inside an object.
			const bool bWriteValueOnly = !Element->bIsKeyValuePair || (Element->Identifier.IsEmpty() && Writer.GetCurrentElementType() != EJson::Object);

			if (Policy::SerializeIfBool(Element, Writer, bWriteValueOnly))
			{
				continue;
			}
			
			if (Policy::SerializeIfNumber(Element, Writer, bWriteValueOnly))
			{
				continue;
			}
			
			if (Policy::SerializeIfString(Element, Writer, bWriteValueOnly))
			{
				continue;
			}
			
			if (Policy::SerializeIfNull(Element, Writer, bWriteValueOnly))
			{
				continue;
			}
			
			if (Policy::SerializeIfArray(ElementStack, Element, Writer, bWriteValueOnly))
			{
				continue;
			}
			
			if (Policy::SerializeIfObject(ElementStack, Element, Writer, bWriteValueOnly))
			{
				continue;
			}
			
			UE_LOGF(LogJson, Fatal, "Could not print Json Value, unrecognized type.");
		}

		if (bCloseWriter)
		{
			return Writer.Close();
		}
		else
		{
			return true;
		}
	}
};

using FJsonSerializer = TJsonSerializer<FJsonSerializerPolicy_JsonObject>;
