// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/JsonTypes.h"
#include "Dom/JsonObject.h"

/**
 * Base interface used to serialize to/from JSON. Hides the fact there are separate read/write classes
 */
struct FJsonSerializerBase
{
	virtual bool IsLoading() const = 0;
	virtual bool IsSaving() const = 0;
	virtual void StartObject() = 0;
	virtual void StartObject(FStringView Name) = 0;
	virtual void EndObject() = 0;
	virtual void StartArray() = 0;
	virtual void StartArray(FStringView Name) = 0;
	virtual void EndArray() = 0;
	virtual void Serialize(FStringView Name, int32& Value) = 0;
	virtual void Serialize(FStringView Name, uint32& Value) = 0;
	virtual void Serialize(FStringView Name, int64& Value) = 0;
	virtual void Serialize(FStringView Name, bool& Value) = 0;
	virtual void Serialize(FStringView Name, FString& Value) = 0;
	virtual void Serialize(FStringView Name, FUtf8String& Value) = 0;
	virtual void Serialize(FStringView Name, FText& Value) = 0;
	virtual void Serialize(FStringView Name, float& Value) = 0;
	virtual void Serialize(FStringView Name, double& Value) = 0;
	virtual void Serialize(FStringView Name, FDateTime& Value) = 0;
	virtual void Serialize(FStringView Name, JsonSimpleValueVariant& InVariant) = 0;
	virtual void SerializeArray(FJsonSerializableArray& Array) = 0;
	virtual void SerializeArray(FStringView Name, FJsonSerializableArray& Value) = 0;
	virtual void SerializeArray(FStringView Name, FJsonSerializableArrayInt& Value) = 0;
	virtual void SerializeArray(FStringView Name, FJsonSerializableArrayFloat& Value) = 0;
	virtual void SerializeMap(FStringView Name, FJsonSerializableKeyValueMap& Map) = 0;
	virtual void SerializeMap(FStringView Name, FJsonSerializableKeyValueMapInt& Map) = 0;
	virtual void SerializeMap(FStringView Name, FJsonSerializableKeyValueMapArrayInt& Map) = 0;
	virtual void SerializeMap(FStringView Name, FJsonSerializableKeyValueMapInt64& Map) = 0;
	virtual void SerializeMap(FStringView Name, FJsonSerializableKeyValueMapFloat& Map) = 0;
	virtual void SerializeMap(FStringView Name, FJsonSerializableKeySimpleValueVariantMap& Map) = 0;
	virtual void SerializeSimpleMap(FJsonSerializableKeyValueMap& Map) = 0;
	virtual void SerializeMapSafe(FStringView Name, FJsonSerializableKeyValueMap& Map) = 0;
	virtual TSharedPtr<FJsonObject> GetObject() = 0;
	virtual void WriteIdentifierPrefix(FStringView Name) = 0;
	virtual void WriteRawJSONValue(FStringView Value) = 0;
	virtual void WriteRawJSONValue(FUtf8StringView Value) = 0;

#if !PLATFORM_TCHAR_IS_UTF8CHAR

	UE_DEPRECATED(5.4, "Passing an ANSI string to StartObject has been deprecated outside of UTF-8 mode. Please use the overload that takes a TCHAR string.")
	void StartObject(FAnsiStringView Name)
	{
		StartObject(StringCast<TCHAR>(Name.GetData(), Name.Len()));
	}

	UE_DEPRECATED(5.4, "Passing an ANSI string to StartArray has been deprecated outside of UTF-8 mode. Please use the overload that takes a TCHAR string.")
	void StartArray(FAnsiStringView Name)
	{
		StartArray(StringCast<TCHAR>(Name.GetData(), Name.Len()));
	}

#endif // !PLATFORM_TCHAR_IS_UTF8CHAR
};
