// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonDataBag.h"
#include "Serialization/JsonTypes.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonSerializable.h"
#include "Serialization/JsonSerializerBase.h"
#include "Serialization/JsonSerializerReader.h"
#include "Serialization/JsonSerializerWriter.h"
#include "JsonGlobals.h"

/**
 * Macros used to generate a serialization function for a class derived from FJsonSerializable
 */
#define BEGIN_JSON_SERIALIZER \
	virtual void Serialize(FJsonSerializerBase& Serializer, bool bFlatObject) override \
	{ \
		if (!bFlatObject) { Serializer.StartObject(); }

#define END_JSON_SERIALIZER \
		if (!bFlatObject) { Serializer.EndObject(); } \
	}

#define JSON_SERIALIZE(JsonName, JsonValue) \
		Serializer.Serialize(TEXTVIEW(JsonName), JsonValue)

#define JSON_SERIALIZE_NAME(JsonName, JsonValue) \
		if (Serializer.IsLoading()) \
		{ \
			FString NameStr; \
			Serializer.Serialize(TEXTVIEW(JsonName), NameStr); \
			JsonValue = FName(*NameStr); \
		} \
		else \
		{ \
			FString NameStr = JsonValue.ToString(); \
			Serializer.Serialize(TEXTVIEW(JsonName), NameStr); \
		}

#define JSON_SERIALIZE_WITHDEFAULT(JsonName, JsonValue, DefaultJsonValue) \
		if (Serializer.IsLoading()) \
		{ \
			if (!Serializer.GetObject()->HasField(TEXTVIEW(JsonName))) \
           	{ \
           		JsonValue = DefaultJsonValue; \
           	} \
        } \
		Serializer.Serialize(TEXTVIEW(JsonName), JsonValue);
		
#define JSON_SERIALIZE_OPTIONAL(JsonName, OptionalJsonValue) \
		if (Serializer.IsLoading()) \
		{ \
			if (Serializer.GetObject()->HasField(TEXTVIEW(JsonName))) \
			{ \
				Serializer.Serialize(TEXTVIEW(JsonName), OptionalJsonValue.Emplace()); \
			} \
		} \
		else \
		{ \
			if (OptionalJsonValue.IsSet()) \
			{ \
				Serializer.Serialize(TEXTVIEW(JsonName), OptionalJsonValue.GetValue()); \
			} \
		}

#define JSON_SERIALIZE_OPTIONAL_MAP(JsonName, JsonMap) \
	if (Serializer.IsLoading() && !JsonMap.IsSet() && Serializer.GetObject().IsValid() && Serializer.GetObject()->HasTypedField<EJson::Object>(TEXTVIEW(JsonName))) \
	{ \
		JsonMap.Emplace(); \
	} \
	\
	if (JsonMap.IsSet()) \
	{ \
		Serializer.SerializeMap(TEXTVIEW(JsonName), JsonMap.GetValue()); \
	}

#define JSON_SERIALIZE_OPTIONAL_ARRAY(JsonName, JsonArray) \
	if (Serializer.IsLoading() && !JsonArray.IsSet() && Serializer.GetObject().IsValid() && Serializer.GetObject()->HasTypedField<EJson::Array>(TEXTVIEW(JsonName))) \
	{ \
		JsonArray.Emplace(); \
	} \
	\
	if (JsonArray.IsSet()) \
	{ \
		Serializer.SerializeArray(TEXTVIEW(JsonName), JsonArray.GetValue()); \
	}

#define JSON_SERIALIZE_ARRAY(JsonName, JsonArray) \
		Serializer.SerializeArray(TEXTVIEW(JsonName), JsonArray)
		
#define JSON_SERIALIZE_ARRAY_WITHDEFAULT(JsonName, JsonArray, DefaultArray) \
		if (Serializer.IsLoading()) \
		{ \
			if (!Serializer.GetObject()->HasField(TEXTVIEW(JsonName))) \
			{ \
				JsonArray = DefaultArray; \
			} \
		} \
		Serializer.SerializeArray(TEXTVIEW(JsonName), JsonArray);
		
#define JSON_SERIALIZE_OBJECT_WITHDEFAULT(JsonName, JsonObject, DefaultObject) \
		if (Serializer.IsLoading()) \
		{ \
			if (!Serializer.GetObject()->HasField(TEXTVIEW(JsonName))) \
			{ \
				JsonObject = DefaultObject; \
			} \
		} \
		Serializer.SerializeArray(TEXTVIEW(JsonName), JsonArray);

#define JSON_SERIALIZE_MAP(JsonName, JsonMap) \
		Serializer.SerializeMap(TEXTVIEW(JsonName), JsonMap)

#define JSON_SERIALIZE_SIMPLECOPY(JsonMap) \
		Serializer.SerializeSimpleMap(JsonMap)

#define JSON_SERIALIZE_MAP_SAFE(JsonName, JsonMap) \
		Serializer.SerializeMapSafe(TEXTVIEW(JsonName), JsonMap)

#define JSON_SERIALIZE_RAW_JSON_STRING(JsonName, JsonValue) \
		if (Serializer.IsLoading()) \
		{ \
			using CharType = TElementType_T<decltype(JsonValue)>; \
			if (Serializer.GetObject()->HasTypedField<EJson::Object>(TEXTVIEW(JsonName))) \
			{ \
				TSharedPtr<FJsonObject> JsonObject = Serializer.GetObject()->GetObjectField(TEXTVIEW(JsonName)); \
				if (JsonObject.IsValid()) \
				{ \
					auto Writer = TJsonWriterFactory<CharType, TCondensedJsonPrintPolicy<CharType>>::Create(&JsonValue); \
					FJsonSerializer::Serialize(JsonObject.ToSharedRef(), Writer); \
				} \
			} \
			else \
			{ \
				JsonValue = TString<CharType>(); \
			} \
		} \
		else \
		{ \
			if (!JsonValue.IsEmpty()) \
			{ \
				Serializer.WriteIdentifierPrefix(TEXTVIEW(JsonName)); \
				Serializer.WriteRawJSONValue(*JsonValue); \
			} \
		}

#define JSON_SERIALIZE_ARRAY_SERIALIZABLE(JsonName, JsonArray, ElementType) \
		if (Serializer.IsLoading()) \
		{ \
			if (Serializer.GetObject()->HasTypedField<EJson::Array>(TEXTVIEW(JsonName))) \
			{ \
				for (auto It = Serializer.GetObject()->GetArrayField(TEXTVIEW(JsonName)).CreateConstIterator(); It; ++It) \
				{ \
					ElementType& Obj = JsonArray.AddDefaulted_GetRef(); \
					Obj.FromJson((*It)->AsObject()); \
				} \
			} \
		} \
		else \
		{ \
			Serializer.StartArray(TEXTVIEW(JsonName)); \
			for (auto It = JsonArray.CreateIterator(); It; ++It) \
			{ \
				It->Serialize(Serializer, false); \
			} \
			Serializer.EndArray(); \
		}
		
#define JSON_SERIALIZE_ARRAY_SERIALIZABLE_WITHDEFAULT(JsonName, JsonArray, ElementType, DefaultArray) \
		if (Serializer.IsLoading()) \
		{ \
			if (Serializer.GetObject()->HasTypedField<EJson::Array>(TEXTVIEW(JsonName))) \
			{ \
				for (auto It = Serializer.GetObject()->GetArrayField(TEXTVIEW(JsonName)).CreateConstIterator(); It; ++It) \
				{ \
					ElementType& Obj = JsonArray.AddDefaulted_GetRef(); \
					Obj.FromJson((*It)->AsObject()); \
				} \
			} \
			else \
			{ \
				JsonArray = DefaultArray; \
			} \
		} \
		else \
		{ \
			Serializer.StartArray(TEXTVIEW(JsonName)); \
			for (auto It = JsonArray.CreateIterator(); It; ++It) \
			{ \
				It->Serialize(Serializer, false); \
			} \
			Serializer.EndArray(); \
		}

#define JSON_SERIALIZE_OPTIONAL_ARRAY_SERIALIZABLE(JsonName, OptionalJsonArray, ElementType) \
		if (Serializer.IsLoading()) \
		{ \
			if (Serializer.GetObject()->HasTypedField<EJson::Array>(TEXTVIEW(JsonName))) \
			{ \
				TArray<ElementType>& JsonArray = OptionalJsonArray.Emplace(); \
				for (auto It = Serializer.GetObject()->GetArrayField(TEXTVIEW(JsonName)).CreateConstIterator(); It; ++It) \
				{ \
					ElementType& Obj = JsonArray.AddDefaulted_GetRef(); \
					Obj.FromJson((*It)->AsObject()); \
				} \
			} \
		} \
		else \
		{ \
			if (OptionalJsonArray.IsSet()) \
			{ \
				Serializer.StartArray(TEXTVIEW(JsonName)); \
				for (auto It = OptionalJsonArray->CreateIterator(); It; ++It) \
				{ \
					It->Serialize(Serializer, false); \
				} \
				Serializer.EndArray(); \
			} \
		}

#define JSON_SERIALIZE_MAP_SERIALIZABLE(JsonName, JsonMap, ElementType) \
		if (Serializer.IsLoading()) \
		{ \
			if (Serializer.GetObject()->HasTypedField<EJson::Object>(TEXTVIEW(JsonName))) \
			{ \
				TSharedPtr<FJsonObject> JsonObj = Serializer.GetObject()->GetObjectField(TEXTVIEW(JsonName)); \
				for (auto MapIt = JsonObj->Values.CreateConstIterator(); MapIt; ++MapIt) \
				{ \
					ElementType NewEntry; \
					NewEntry.FromJson(MapIt.Value()->AsObject()); \
					JsonMap.Add(FString(MapIt.Key()), NewEntry); \
				} \
			} \
		} \
		else \
		{ \
			Serializer.StartObject(TEXTVIEW(JsonName)); \
			for (auto It = JsonMap.CreateIterator(); It; ++It) \
			{ \
				Serializer.StartObject(MakeStringView(It.Key())); \
				It.Value().Serialize(Serializer, true); \
				Serializer.EndObject(); \
			} \
			Serializer.EndObject(); \
		}

#define JSON_SERIALIZE_MAP_ARRAY_SERIALIZABLE(JsonName, JsonMap, ElementType) \
		if (Serializer.IsLoading()) \
		{ \
			if (Serializer.GetObject()->HasTypedField<EJson::Object>(TEXTVIEW(JsonName))) \
			{ \
				TSharedPtr<FJsonObject> JsonObj = Serializer.GetObject()->GetObjectField(TEXTVIEW(JsonName)); \
				for (auto MapIt = JsonObj->Values.CreateConstIterator(); MapIt; ++MapIt) \
				{ \
					TArray<ElementType> NewEntry; \
					for (auto ArrayIt : MapIt.Value()->AsArray()) \
					{ \
						NewEntry.Add_GetRef(ElementType()).FromJson(ArrayIt->AsObject()); \
					} \
					JsonMap.Add(FString(MapIt.Key()), NewEntry); \
				} \
			} \
		} \
		else \
		{ \
			Serializer.StartObject(TEXTVIEW(JsonName)); \
			for (auto It = JsonMap.CreateIterator(); It; ++It) \
			{ \
				Serializer.StartArray(It.Key()); \
				for (auto ArrayEntry : It.Value()) \
				{ \
					ArrayEntry.Serialize(Serializer, false); \
				} \
				Serializer.EndArray(); \
			} \
			Serializer.EndObject(); \
		}

// Use this to read/write JsonValue with its own keys and values corresponding to its members
#define JSON_SERIALIZE_MEMBERS_OF(JsonValue) \
		JsonValue.Serialize(Serializer, /*bIsFlatObject*/true)

#define JSON_SERIALIZE_SERIALIZABLE(JsonName, JsonValue) UE_DEPRECATED_MACRO(5.6, "JSON_SERIALIZE_SERIALIZABLE has been deprecated, use JSON_SERIALIZE_MEMBERS_OF instead") JSON_SERIALIZE_MEMBERS_OF(JsonValue)

// Use this when JsonSerializableObject will be read/write as json object, while JsonName is the key
#define JSON_SERIALIZE_OBJECT_SERIALIZABLE(JsonName, JsonSerializableObject) \
		/* Process the JsonName field differently because it is an object */ \
		if (Serializer.IsLoading()) \
		{ \
			/* Read in the value from the JsonName field */ \
			if (Serializer.GetObject()->HasTypedField<EJson::Object>(TEXTVIEW(JsonName))) \
			{ \
				TSharedPtr<FJsonObject> JsonObj = Serializer.GetObject()->GetObjectField(TEXTVIEW(JsonName)); \
				if (JsonObj.IsValid()) \
				{ \
					(JsonSerializableObject).FromJson(JsonObj); \
				} \
			} \
		} \
		else \
		{ \
			/* Write the value to the Name field */ \
			Serializer.StartObject(TEXTVIEW(JsonName)); \
			(JsonSerializableObject).Serialize(Serializer, true); \
			Serializer.EndObject(); \
		}
		
#define JSON_SERIALIZE_OBJECT_SERIALIZABLE_WITHDEFAULT(JsonName, JsonSerializableObject, JsonSerializableObjectDefault) \
		/* Process the JsonName field differently because it is an object */ \
		if (Serializer.IsLoading()) \
		{ \
			/* Read in the value from the JsonName field */ \
			if (Serializer.GetObject()->HasTypedField<EJson::Object>(TEXTVIEW(JsonName))) \
			{ \
				TSharedPtr<FJsonObject> JsonObj = Serializer.GetObject()->GetObjectField(TEXTVIEW(JsonName)); \
				if (JsonObj.IsValid()) \
				{ \
					(JsonSerializableObject).FromJson(JsonObj); \
				} \
			} \
			else \
			{ \
				JsonSerializableObject = JsonSerializableObjectDefault; \
			} \
		} \
		else \
		{ \
			/* Write the value to the Name field */ \
			Serializer.StartObject(TEXTVIEW(JsonName)); \
			(JsonSerializableObject).Serialize(Serializer, true); \
			Serializer.EndObject(); \
		}

#define JSON_SERIALIZE_OPTIONAL_OBJECT_SERIALIZABLE(JsonName, JsonSerializableObject) \
		if (Serializer.IsLoading()) \
		{ \
			using ObjectType = TRemoveReference<decltype(JsonSerializableObject.GetValue())>::Type; \
			if (Serializer.GetObject()->HasTypedField<EJson::Object>(TEXTVIEW(JsonName))) \
			{ \
				TSharedPtr<FJsonObject> JsonObj = Serializer.GetObject()->GetObjectField(TEXTVIEW(JsonName)); \
				if (JsonObj.IsValid()) \
				{ \
					JsonSerializableObject = ObjectType{}; \
					JsonSerializableObject.GetValue().FromJson(JsonObj); \
				} \
			} \
		} \
		else \
		{ \
			if (JsonSerializableObject.IsSet()) \
			{ \
				Serializer.StartObject(TEXTVIEW(JsonName)); \
				(JsonSerializableObject.GetValue()).Serialize(Serializer, true); \
				Serializer.EndObject(); \
			} \
		}

#define JSON_SERIALIZE_SHARED_OBJECT_SERIALIZABLE(JsonName, JsonSerializableObject) \
		if (Serializer.IsLoading()) \
		{ \
			using ObjectType = TRemoveReference<decltype(*JsonSerializableObject)>::Type; \
			if (Serializer.GetObject()->HasTypedField<EJson::Object>(TEXTVIEW(JsonName))) \
			{ \
				TSharedPtr<FJsonObject> JsonObj = Serializer.GetObject()->GetObjectField(TEXTVIEW(JsonName)); \
				if (JsonObj.IsValid()) \
				{ \
					JsonSerializableObject = MakeShared<ObjectType>(); \
					JsonSerializableObject->FromJson(JsonObj); \
				} \
			} \
		} \
		else \
		{ \
			if (JsonSerializableObject) \
			{ \
				Serializer.StartObject(TEXTVIEW(JsonName)); \
				JsonSerializableObject->Serialize(Serializer, true); \
				Serializer.EndObject(); \
			} \
		}

#define JSON_SERIALIZE_DATETIME_UNIX_TIMESTAMP(JsonName, JsonDateTime) \
		if (Serializer.IsLoading()) \
		{ \
			int64 UnixTimestampValue; \
			Serializer.Serialize(TEXTVIEW(JsonName), UnixTimestampValue); \
			JsonDateTime = FDateTime::FromUnixTimestamp(UnixTimestampValue); \
		} \
		else \
		{ \
			int64 UnixTimestampValue = JsonDateTime.ToUnixTimestamp(); \
			Serializer.Serialize(TEXTVIEW(JsonName), UnixTimestampValue); \
		}

#define JSON_SERIALIZE_DATETIME_UNIX_TIMESTAMP_MILLISECONDS(JsonName, JsonDateTime) \
if (Serializer.IsLoading()) \
{ \
	int64 UnixTimestampValueInMilliseconds; \
	Serializer.Serialize(TEXTVIEW(JsonName), UnixTimestampValueInMilliseconds); \
	JsonDateTime = FDateTime::FromUnixTimestamp(UnixTimestampValueInMilliseconds / 1000); \
} \
else \
{ \
	int64 UnixTimestampValueInMilliseconds = JsonDateTime.ToUnixTimestamp() * 1000; \
	Serializer.Serialize(TEXTVIEW(JsonName), UnixTimestampValueInMilliseconds); \
}

#define JSON_SERIALIZE_ENUM(JsonName, JsonEnum) \
if (Serializer.IsLoading()) \
{ \
	FString JsonTextValue; \
	Serializer.Serialize(TEXTVIEW(JsonName), JsonTextValue); \
	LexFromString(JsonEnum, *JsonTextValue); \
} \
else \
{ \
	FString JsonTextValue = LexToString(JsonEnum); \
	Serializer.Serialize(TEXTVIEW(JsonName), JsonTextValue); \
}

#define JSON_SERIALIZE_ENUM_ARRAY(JsonName, JsonArray, EnumType) \
if (Serializer.IsLoading()) \
{ \
	if (Serializer.GetObject()->HasTypedField<EJson::Array>(TEXTVIEW(JsonName))) \
	{ \
		EnumType EnumValue; \
		for (auto It = Serializer.GetObject()->GetArrayField(TEXTVIEW(JsonName)).CreateConstIterator(); It; ++It) \
		{ \
			LexFromString(EnumValue, *(*It)->AsString()); \
			JsonArray.Add(EnumValue); \
		} \
	} \
} \
else \
{ \
	Serializer.StartArray(TEXTVIEW(JsonName)); \
	for (EnumType& EnumValue : JsonArray) \
	{ \
		FString JsonTextValue = LexToString(EnumValue); \
		Serializer.Serialize(TEXTVIEW(JsonName), JsonTextValue); \
	} \
	Serializer.EndArray(); \
}

/**
 * Private macros for Variant JSON serialization
 */
#define JSON_SERIALIZE_PRIVATE_VARIANT_BEGIN(JsonName, JsonValue) \
	TSharedPtr<FJsonObject> VariantJsonObj; \
	FString VariantTypeStr; \
	if (VariantValue != nullptr) \
	{ \
		if (Serializer.IsLoading()) \
		{ \
			if (Serializer.GetObject()->HasTypedField<EJson::Object>(TEXTVIEW(JsonName))) \
			{ \
				VariantJsonObj = Serializer.GetObject()->GetObjectField(TEXTVIEW(JsonName)); \
				if (VariantJsonObj.IsValid()) \
				{ \
					VariantTypeStr = VariantJsonObj->GetStringField(TEXTVIEW("Type")); \
					const TSharedPtr<FJsonObject>* VariantValueObject = nullptr; \
					if (VariantJsonObj->TryGetObjectField(TEXTVIEW("Value"), VariantValueObject); VariantValueObject != nullptr) \
					{ \
						VariantJsonObj = *VariantValueObject; \
					} \
					else \
					{ \
						VariantJsonObj = {}; \
					} \
				} \
			} \
		} \
		else \
		{ \
			Serializer.StartObject(TEXTVIEW(JsonName)); \
		} \
	}

#define JSON_SERIALIZE_PRIVATE_VARIANT_END() \
	if (VariantValue != nullptr) \
	{ \
		if (!Serializer.IsLoading()) \
		{ \
			Serializer.EndObject(); \
		} \
	}

/**
 * Macros for begin/end of regular TVariant<...> value, insert JSON_SERIALIZE_VARIANT_IFTYPE_SERIALIZABLE in between BEGIN/END pair
 */
#define JSON_SERIALIZE_VARIANT_BEGIN(JsonName, JsonValue) \
	if (true) \
	{ \
		auto* VariantValue = &JsonValue; \
		JSON_SERIALIZE_PRIVATE_VARIANT_BEGIN(JsonName, JsonValue)

#define JSON_SERIALIZE_VARIANT_END() \
		JSON_SERIALIZE_PRIVATE_VARIANT_END() \
	}

/**
 * Macros for begin/end of TOptional<TVariant<...>> value, insert JSON_SERIALIZE_VARIANT_IFTYPE_SERIALIZABLE in between BEGIN/END pair
 */
#define JSON_SERIALIZE_OPTIONAL_VARIANT_BEGIN(JsonName, JsonValue) \
	if (true) \
	{ \
		if (Serializer.IsLoading()) \
		{ \
			if (Serializer.GetObject()->HasTypedField<EJson::Object>(TEXTVIEW(JsonName))) \
			{ \
				JsonValue.Emplace(); \
			} \
		} \
		auto* VariantValue = JsonValue.IsSet() ? &JsonValue.GetValue() : nullptr; \
		JSON_SERIALIZE_PRIVATE_VARIANT_BEGIN(JsonName, JsonValue)

#define JSON_SERIALIZE_OPTIONAL_VARIANT_END() \
		JSON_SERIALIZE_PRIVATE_VARIANT_END() \
	}

/**
 * Macros for serializing a specific sub type of a variant, only to be insert in between (optional) VARIANT begin/end pair
 */
#define JSON_SERIALIZE_VARIANT_IFTYPE_SERIALIZABLE(TypeName, Type) \
		if (VariantValue != nullptr) \
		{ \
			if (Serializer.IsLoading()) \
			{ \
				if (VariantJsonObj.IsValid() && VariantTypeStr == TEXTVIEW(TypeName)) \
				{ \
					VariantValue->Emplace<Type>(); \
					VariantValue->Get<Type>().FromJson(VariantJsonObj); \
				} \
			} \
			else \
			{ \
				/* created Value + Type */ \
				if (VariantValue->IsType<Type>()) \
				{ \
					VariantTypeStr = TEXTVIEW(TypeName); \
					Serializer.Serialize(TEXTVIEW("Type"), VariantTypeStr); \
					Serializer.StartObject(TEXTVIEW("Value")); \
					VariantValue->Get<Type>().Serialize(Serializer, true); \
					Serializer.EndObject(); \
				} \
			} \
		}
