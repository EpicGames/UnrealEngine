// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/JsonSerializerBase.h"
#include "Dom/JsonObject.h"

/**
 * Implements the abstract serializer interface hiding the underlying reader object
 */
class FJsonSerializerReader : public FJsonSerializerBase
{
public:
	/**
	 * Inits the base JSON object that is being read from
	 *
	 * @param InJsonObject the JSON object to serialize from
	 */
	JSON_API FJsonSerializerReader(TSharedPtr<FJsonObject> InJsonObject);

	JSON_API virtual ~FJsonSerializerReader();

	/** Is the JSON being read from */
	JSON_API virtual bool IsLoading() const override;
	/** Is the JSON being written to */
	JSON_API virtual bool IsSaving() const override;
	/** Access to the root Json object being read */
	JSON_API virtual TSharedPtr<FJsonObject> GetObject() override;

	/** Ignored */
	JSON_API virtual void StartObject() override;
	/** Ignored */
	JSON_API virtual void StartObject(FStringView Name) override;
	/** Ignored */
	JSON_API virtual void EndObject() override;
	/** Ignored */
	JSON_API virtual void StartArray() override;
	/** Ignored */
	JSON_API virtual void StartArray(FStringView Name) override;
	/** Ignored */
	JSON_API virtual void EndArray() override;
	/**
	 * If the underlying json object has the field, it is read into the value
	 *
	 * @param Name the name of the field to read
	 * @param Value the out value to read the data into
	 */
	JSON_API virtual void Serialize(FStringView Name, int32& Value) override;
	/**
	 * If the underlying json object has the field, it is read into the value
	 *
	 * @param Name the name of the field to read
	 * @param Value the out value to read the data into
	 */
	JSON_API virtual void Serialize(FStringView Name, uint32& Value) override;
	/**
	 * If the underlying json object has the field, it is read into the value
	 *
	 * @param Name the name of the field to read
	 * @param Value the out value to read the data into
	 */
	JSON_API virtual void Serialize(FStringView Name, int64& Value) override;
	/**
	 * If the underlying json object has the field, it is read into the value
	 *
	 * @param Name the name of the field to read
	 * @param Value the out value to read the data into
	 */
	JSON_API virtual void Serialize(FStringView Name, bool& Value) override;
	/**
	 * If the underlying json object has the field, it is read into the value
	 *
	 * @param Name the name of the field to read
	 * @param Value the out value to read the data into
	 */
	JSON_API virtual void Serialize(FStringView Name, FString& Value) override;
	/**
	 * If the underlying json object has the field, it is read into the value
	 *
	 * @param Name the name of the field to read
	 * @param Value the out value to read the data into
	 */
	JSON_API virtual void Serialize(FStringView Name, FUtf8String& Value) override;
	/**
	 * If the underlying json object has the field, it is read into the value
	 *
	 * @param Name the name of the field to read
	 * @param Value the out value to read the data into
	 */
	JSON_API virtual void Serialize(FStringView Name, FText& Value) override;
	/**
	 * If the underlying json object has the field, it is read into the value
	 *
	 * @param Name the name of the field to read
	 * @param Value the out value to read the data into
	 */
	JSON_API virtual void Serialize(FStringView Name, float& Value) override;
	/**
	* If the underlying json object has the field, it is read into the value
	*
	* @param Name the name of the field to read
	* @param Value the out value to read the data into
	*/
	JSON_API virtual void Serialize(FStringView Name, double& Value) override;
	/**
	* Writes the field name and the corresponding value to the JSON data
	*
	* @param Name the field name to write out
	* @param Value the value to write out
	*/
	JSON_API virtual void Serialize(FStringView Name, FDateTime& Value) override;
	/**
	* Writes the field name and the corresponding value to the JSON data
	*
	* @param Name the field name to write out
	* @param Value the value to write out
	*/
	JSON_API virtual void Serialize(FStringView Name, JsonSimpleValueVariant& InVariant) override;

	/**
	 * Serializes an array of values
	 *
	 * @param Name the name of the property to serialize
	 * @param Array the array to serialize
	 */
	JSON_API virtual void SerializeArray(FJsonSerializableArray& Array) override;
	/**
	 * Serializes an array of values with an identifier
	 *
	 * @param Name the name of the property to serialize
	 * @param Array the array to serialize
	 */
	JSON_API virtual void SerializeArray(FStringView Name, FJsonSerializableArray& Array) override;
	/**
	 * Serializes an array of values with an identifier
	 *
	 * @param Name the name of the property to serialize
	 * @param Array the array to serialize
	 */
	JSON_API virtual void SerializeArray(FStringView Name, FJsonSerializableArrayInt& Array) override;
	/**
	 * Serializes an array of values with an identifier
	 *
	 * @param Name the name of the property to serialize
	 * @param Array the array to serialize
	 */
	JSON_API virtual void SerializeArray(FStringView Name, FJsonSerializableArrayFloat& Array) override;
	/**
	 * Serializes the keys & values for map
	 *
	 * @param Name the name of the property to serialize
	 * @param Map the map to serialize
	 */
	JSON_API virtual void SerializeMap(FStringView Name, FJsonSerializableKeyValueMap& Map) override;

	/**
	 * Serializes the keys & values for map
	 *
	 * @param Name the name of the property to serialize
	 * @param Map the map to serialize
	 */
	JSON_API virtual void SerializeMap(FStringView Name, FJsonSerializableKeyValueMapInt& Map) override;
	
	/**
     * Serializes the keys & values for map
     *
     * @param Name the name of the property to serialize
     * @param Map the map to serialize
     */
    JSON_API virtual void SerializeMap(FStringView Name, FJsonSerializableKeyValueMapArrayInt& Map) override;

	/**
	 * Serializes the keys & values for map
	 *
	 * @param Name the name of the property to serialize
	 * @param Map the map to serialize
	 */
	JSON_API virtual void SerializeMap(FStringView Name, FJsonSerializableKeyValueMapInt64& Map) override;

	/**
	 * Serializes the keys & values for map
	 *
	 * @param Name the name of the property to serialize
	 * @param Map the map to serialize
	 */
	JSON_API virtual void SerializeMap(FStringView Name, FJsonSerializableKeyValueMapFloat& Map) override;

	/**
	 * Serializes the keys & values for map
	 *
	 * @param Name the name of the property to serialize
	 * @param Map the map to serialize
	 */
	JSON_API virtual void SerializeMap(FStringView Name, FJsonSerializableKeySimpleValueVariantMap& Map) override;

	JSON_API virtual void SerializeSimpleMap(FJsonSerializableKeyValueMap& Map) override;

	/**
	 * Deserializes keys and values from an object into a map, but only if the value is trivially convertable to string.
	 *
	 * @param Name Name of property to deserialize
	 * @param Map The Map to fill with String values found
	 */
	JSON_API virtual void SerializeMapSafe(FStringView Name, FJsonSerializableKeyValueMap& Map) override;

	JSON_API virtual void WriteIdentifierPrefix(FStringView Name) override;

	JSON_API virtual void WriteRawJSONValue(FStringView Value) override;

	JSON_API virtual void WriteRawJSONValue(FUtf8StringView Value) override;
private:
	/** The object that holds the parsed JSON data */
	TSharedPtr<FJsonObject> JsonObject;
};

