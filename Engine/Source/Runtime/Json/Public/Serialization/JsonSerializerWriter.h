// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Policies/PrettyJsonPrintPolicy.h"
#include "Serialization/JsonTypes.h"
#include "Serialization/JsonSerializerBase.h"
#include "Serialization/JsonWriter.h"

/**
 * Implements the abstract serializer interface hiding the underlying writer object
 */
template <class CharType = TCHAR, class PrintPolicy = TPrettyJsonPrintPolicy<CharType> >
class FJsonSerializerWriter :
	public FJsonSerializerBase
{
	/** The object to write the JSON output to */
	TSharedRef<TJsonWriter<CharType, PrintPolicy> > JsonWriter;

public:

	/**
	 * Initializes the writer object
	 *
	 * @param InJsonWriter the object to write the JSON output to
	 */
	FJsonSerializerWriter(TSharedRef<TJsonWriter<CharType, PrintPolicy> > InJsonWriter) :
		JsonWriter(InJsonWriter)
	{
	}

	virtual ~FJsonSerializerWriter() = default;

	/** Is the JSON being read from */
	virtual bool IsLoading() const override { return false; }
	/** Is the JSON being written to */
	virtual bool IsSaving() const override { return true; }
	/** Access to the root object */
	virtual TSharedPtr<FJsonObject> GetObject() override { return TSharedPtr<FJsonObject>(); }

	/**
	 * Starts a new object "{"
	 */
	virtual void StartObject() override
	{
		JsonWriter->WriteObjectStart();
	}

	/**
	 * Starts a new object "{"
	 */
	virtual void StartObject(FStringView Name) override
	{
		JsonWriter->WriteObjectStart(Name);
	}
	/**
	 * Completes the definition of an object "}"
	 */
	virtual void EndObject() override
	{
		JsonWriter->WriteObjectEnd();
	}

	virtual void StartArray() override
	{
		JsonWriter->WriteArrayStart();
	}

	virtual void StartArray(FStringView Name) override
	{
		JsonWriter->WriteArrayStart(Name);
	}

	virtual void EndArray() override
	{
		JsonWriter->WriteArrayEnd();
	}
	/**
	 * Writes the field name and the corresponding value to the JSON data
	 *
	 * @param Name the field name to write out
	 * @param Value the value to write out
	 */
	virtual void Serialize(FStringView Name, int32& Value) override
	{
		JsonWriter->WriteValue(Name, Value);
	}
	/**
	 * Writes the field name and the corresponding value to the JSON data
	 *
	 * @param Name the field name to write out
	 * @param Value the value to write out
	 */
	virtual void Serialize(FStringView Name, uint32& Value) override
	{
		JsonWriter->WriteValue(Name, static_cast<int64>(Value));
	}
	/**
	 * Writes the field name and the corresponding value to the JSON data
	 *
	 * @param Name the field name to write out
	 * @param Value the value to write out
	 */
	virtual void Serialize(FStringView Name, int64& Value) override
	{
		JsonWriter->WriteValue(Name, Value);
	}
	/**
	 * Writes the field name and the corresponding value to the JSON data
	 *
	 * @param Name the field name to write out
	 * @param Value the value to write out
	 */
	virtual void Serialize(FStringView Name, bool& Value) override
	{
		JsonWriter->WriteValue(Name, Value);
	}
	/**
	 * Writes the field name and the corresponding value to the JSON data
	 *
	 * @param Name the field name to write out
	 * @param Value the value to write out
	 */
	virtual void Serialize(FStringView Name, FString& Value) override
	{
		JsonWriter->WriteValue(Name, Value);
	}
	/**
	* Writes the field name and the corresponding value to the JSON data
	*
	* @param Name the field name to write out
	* @param Value the value to write out
	*/
	virtual void Serialize(FStringView Name, FUtf8String& Value) override
	{
		JsonWriter->WriteValue(Name, Value);
	}
	/**
	 * Writes the field name and the corresponding value to the JSON data
	 *
	 * @param Name the field name to write out
	 * @param Value the value to write out
	 */
	virtual void Serialize(FStringView Name, FText& Value) override
	{
		JsonWriter->WriteValue(Name, Value.ToString());
	}
	/**
	 * Writes the field name and the corresponding value to the JSON data
	 *
	 * @param Name the field name to write out
	 * @param Value the value to write out
	 */
	virtual void Serialize(FStringView Name, float& Value) override
	{
		JsonWriter->WriteValue(Name, Value);
	}
	/**
	* Writes the field name and the corresponding value to the JSON data
	*
	* @param Name the field name to write out
	* @param Value the value to write out
	*/
	virtual void Serialize(FStringView Name, double& Value) override
	{
		JsonWriter->WriteValue(Name, Value);
	}
	/**
	* Writes the field name and the corresponding value to the JSON data
	*
	* @param Name the field name to write out
	* @param Value the value to write out
	*/
	virtual void Serialize(FStringView Name, FDateTime& Value) override
	{
		if (Value.GetTicks() > 0)
		{
			JsonWriter->WriteValue(Name, Value.ToIso8601());
		}
	}

	virtual void Serialize(FStringView Name, JsonSimpleValueVariant& InVariant) override
	{
		::Visit([this, &Name](auto& StoredValue)
			{
				using StoredValueType = std::decay_t<decltype(StoredValue)>;
				if constexpr (std::is_same_v<StoredValueType, JsonNumberValueVariants>)
				{
					::Visit([this, &Name](auto& StoredNumberValue)
						{
							JsonWriter->WriteValue(Name, StoredNumberValue);
						}, StoredValue);
				}
				else
				{
					JsonWriter->WriteValue(Name, StoredValue);
				}
			}, InVariant);
	}

	/**
	 * Serializes an array of values
	 *
	 * @param Name the name of the property to serialize
	 * @param Array the array to serialize
	 */
	virtual void SerializeArray(FJsonSerializableArray& Array) override
	{
		JsonWriter->WriteArrayStart();
		// Iterate all of values
		for (FJsonSerializableArray::TIterator ArrayIt(Array); ArrayIt; ++ArrayIt)
		{
			JsonWriter->WriteValue(*ArrayIt);
		}
		JsonWriter->WriteArrayEnd();
	}
	/**
	 * Serializes an array of values with an identifier
	 *
	 * @param Name the name of the property to serialize
	 * @param Array the array to serialize
	 */
	virtual void SerializeArray(FStringView Name, FJsonSerializableArray& Array) override
	{
		JsonWriter->WriteArrayStart(Name);
		// Iterate all of values
		for (FJsonSerializableArray::ElementType& Item :  Array)
		{
			JsonWriter->WriteValue(Item);
		}
		JsonWriter->WriteArrayEnd();
	}
	/**
	 * Serializes an array of values with an identifier
	 *
	 * @param Name the name of the property to serialize
	 * @param Array the array to serialize
	 */
	virtual void SerializeArray(FStringView Name, FJsonSerializableArrayInt& Array) override
	{
		JsonWriter->WriteArrayStart(Name);
		// Iterate all of values
		for (FJsonSerializableArrayInt::ElementType& Item : Array)
		{
			JsonWriter->WriteValue(Item);
		}
		JsonWriter->WriteArrayEnd();
	}

	/**
	 * Serializes an array of values with an identifier
	 *
	 * @param Name the name of the property to serialize
	 * @param Array the array to serialize
	 */
	virtual void SerializeArray(FStringView Name, FJsonSerializableArrayFloat& Array) override
	{
		JsonWriter->WriteArrayStart(Name);
		// Iterate all of values
		for (FJsonSerializableArrayFloat::ElementType& Item : Array)
		{
			JsonWriter->WriteValue(Item);
		}
		JsonWriter->WriteArrayEnd();
	}

	/**
	 * Serializes the keys & values for map
	 *
	 * @param Name the name of the property to serialize
	 * @param Map the map to serialize
	 */
	virtual void SerializeMap(FStringView Name, FJsonSerializableKeyValueMap& Map) override
	{
		JsonWriter->WriteObjectStart(Name);
		// Iterate all of the keys and their values
		for (FJsonSerializableKeyValueMap::ElementType& Pair : Map)
		{
			Serialize(*Pair.Key, Pair.Value);
		}
		JsonWriter->WriteObjectEnd();
	}

	/**
	 * Serializes the keys & values for map
	 *
	 * @param Name the name of the property to serialize
	 * @param Map the map to serialize
	 */
	virtual void SerializeMap(FStringView Name, FJsonSerializableKeyValueMapInt& Map) override
	{
		JsonWriter->WriteObjectStart(Name);
		// Iterate all of the keys and their values
		for (FJsonSerializableKeyValueMapInt::ElementType& Pair : Map)
		{
			Serialize(*Pair.Key, Pair.Value);
		}
		JsonWriter->WriteObjectEnd();
	}
	/**
	 * Serializes the keys & values for map
	 *
	 * @param Name the name of the property to serialize
	 * @param Map the map to serialize
	 */
	virtual void SerializeMap(FStringView Name, FJsonSerializableKeyValueMapArrayInt& Map) override
	{
		JsonWriter->WriteObjectStart(Name);
		// Iterate all of the keys and their values
		for (FJsonSerializableKeyValueMapArrayInt::ElementType& Pair : Map)
		{
			SerializeArray(*Pair.Key, Pair.Value);
		}
		JsonWriter->WriteObjectEnd();
	}

	/**
	 * Serializes the keys & values for map
	 *
	 * @param Name the name of the property to serialize
	 * @param Map the map to serialize
	 */
	virtual void SerializeMap(FStringView Name, FJsonSerializableKeyValueMapInt64& Map) override
	{
		JsonWriter->WriteObjectStart(Name);
		// Iterate all of the keys and their values
		for (FJsonSerializableKeyValueMapInt64::ElementType& Pair : Map)
		{
			Serialize(*Pair.Key, Pair.Value);
		}
		JsonWriter->WriteObjectEnd();
	}

	/**
	 * Serializes the keys & values for map
	 *
	 * @param Name the name of the property to serialize
	 * @param Map the map to serialize
	 */
	virtual void SerializeMap(FStringView Name, FJsonSerializableKeyValueMapFloat& Map) override
	{
		JsonWriter->WriteObjectStart(Name);
		// Iterate all of the keys and their values
		for (FJsonSerializableKeyValueMapFloat::ElementType& Pair : Map)
		{
			Serialize(*Pair.Key, Pair.Value);
		}
		JsonWriter->WriteObjectEnd();
	}

	virtual void SerializeMap(FStringView Name, FJsonSerializableKeySimpleValueVariantMap& Map) override
	{
		JsonWriter->WriteObjectStart(Name);
		// Iterate all of the keys and their values
		for (TPair<FString, JsonSimpleValueVariant>& Pair : Map)
		{
			::Visit([this, &Pair](auto& StoredValue)
				{
					using StoredValueType = std::decay_t<decltype(StoredValue)>;
					if constexpr (std::is_same_v<StoredValueType, JsonNumberValueVariants>)
					{
						::Visit([this, &Pair](auto& StoredNumberValue)
							{
								Serialize(*Pair.Key, StoredNumberValue);
							}, StoredValue);
					}
					else
					{
						Serialize(*Pair.Key, StoredValue);
					}
				}, Pair.Value);
		}
		JsonWriter->WriteObjectEnd();
	}

	virtual void SerializeSimpleMap(FJsonSerializableKeyValueMap& Map) override
	{
		// writing does nothing here, this is meant to read in all data from a json object 
		// writing is explicitly handled per key/type
	}

	/**
	 * Serializes keys and values from an object into a map.
	 *
	 * @param Name Name of property to serialize
	 * @param Map The Map to copy String values from
	 */
	virtual void SerializeMapSafe(FStringView Name, FJsonSerializableKeyValueMap& Map)
	{
		SerializeMap(Name, Map);
	}

	virtual void WriteIdentifierPrefix(FStringView Name)
	{
		JsonWriter->WriteIdentifierPrefix(Name);
	}

	virtual void WriteRawJSONValue(FStringView Value)
	{
		JsonWriter->WriteRawJSONValue(Value);
	}

	virtual void WriteRawJSONValue(FUtf8StringView Value)
	{
		JsonWriter->WriteRawJSONValue(Value);
	}
};

