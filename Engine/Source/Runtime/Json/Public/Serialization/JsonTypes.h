// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/TVariant.h"

class Error;

/**
 * Json (JavaScript Object Notation) is a lightweight data-interchange format.
 * Information on how it works can be found here: http://www.json.org/.
 * This code was written from scratch with only the Json spec as a guide.
 *
 * In order to use Json effectively, you need to be familiar with the Object/Value
 * hierarchy, and you should use the FJsonObject class and FJsonValue subclasses.
 */

/**
 * Represents all the types a Json Value can be.
 */
enum class EJson
{
	None,
	Null,
	String,
	Number,
	Boolean,
	Array,
	Object
};


enum class EJsonToken
{
	None,
	Comma,
	CurlyOpen,
	CurlyClose,
	SquareOpen,
	SquareClose,
	Colon,
	String,

	// short values
	Number,
	True,
	False,
	Null,

	Identifier
};

inline bool EJsonToken_IsShortValue(EJsonToken Token)
{
	return Token >= EJsonToken::Number && Token <= EJsonToken::Null;
}

enum class EJsonNotation
{
	ObjectStart,
	ObjectEnd,
	ArrayStart,
	ArrayEnd,
	Boolean,
	String,
	Number,
	Null,
	Error
};

/** Array of data */
using FJsonSerializableArray = TArray<FString>;
using FJsonSerializableArrayInt = TArray<int32>;
using FJsonSerializableArrayFloat = TArray<float>;

/** Maps a key to a value */
using FJsonSerializableKeyValueMap = TMap<FString, FString>;
using FJsonSerializableKeyValueMapInt = TMap<FString, int32>;
using FJsonSerializableKeyValueMapInt64 = TMap<FString, int64>;
using FJsonSerializableKeyValueMapFloat = TMap<FString, float>;
using FJsonSerializableKeyValueMapArrayInt = TMap<FString, FJsonSerializableArrayInt>;

/** We always read with the highest precisions (int64 or double), but we can write with any level of precision */
using JsonNumberValueVariants = TVariant<int32, uint32, int64, float, double>; 

using JsonSimpleValueVariant = TVariant<bool /* EJson::Boolean */, JsonNumberValueVariants /* EJson::Number */, FString /* EJson::String */>;

/** Helps keep key values unique */
using FJsonSerializableKeySimpleValueVariantMap = TMap<FString, JsonSimpleValueVariant>;