// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Utf8String.h"
#include "CoreMinimal.h"
#include "Serialization/JsonTypes.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "Serialization/MemoryWriter.h"

/**
* Takes an input source char representing and returns if it is possible to represent in DstChar expected encoding
* 
* @param Char source char
* @return false if character is a control char or if it is out of range of representation in case of converting to ANSICHAR. true otherwise
**/
template <typename DstChar, typename SrcChar>
bool HasDestinationJsonStringCharRepresentation(SrcChar Char)
{
	return Char >= CHARTEXT(SrcChar, ' ') && (std::is_same_v<DstChar, SrcChar> || !std::is_same_v<DstChar, ANSICHAR> || Char <= (SrcChar)0x7E);
}

/**
 * Takes an input string and escapes it so it can be written as a valid Json string. Also adds the quotes.
 * Appends to a given string-like object to avoid reallocations.
 * String-like object must support operator+=(const TCHAR*) and operation+=(TCHAR)
 *
 * @param AppendTo the string to append to.
 * @param StringVal the string to escape
 * @return the AppendTo string for convenience.
 */
template<typename StringType>
inline StringType& AppendEscapeJsonString(StringType& AppendTo, const FString& StringVal)
{
	using CharType = typename StringType::ElementType;
	AppendTo += TEXT("\"");
	for (const CharType* Char = *StringVal; *Char != CHARTEXT(CharType, '\0'); ++Char)
	{
		switch (*Char)
		{
		case CHARTEXT(CharType, '\\'): AppendTo += CHARTEXT(CharType, "\\\\"); break;
		case CHARTEXT(CharType, '\n'): AppendTo += CHARTEXT(CharType, "\\n"); break;
		case CHARTEXT(CharType, '\t'): AppendTo += CHARTEXT(CharType, "\\t"); break;
		case CHARTEXT(CharType, '\b'): AppendTo += CHARTEXT(CharType, "\\b"); break;
		case CHARTEXT(CharType, '\f'): AppendTo += CHARTEXT(CharType, "\\f"); break;
		case CHARTEXT(CharType, '\r'): AppendTo += CHARTEXT(CharType, "\\r"); break;
		case CHARTEXT(CharType, '\"'): AppendTo += CHARTEXT(CharType, "\\\""); break;
		default:
			// Must escape control characters or non representable characters
			if (HasDestinationJsonStringCharRepresentation<CharType>(*Char) )
			{
				AppendTo += *Char;
			}
			else
			{
				AppendTo.Appendf(CHARTEXT(CharType, "\\u%04x"), *Char);
			}
		}
	}
	AppendTo += CHARTEXT(CharType, "\"");

	return AppendTo;
}

/**
 * Takes an input string and escapes it so it can be written as a valid Json string. Also adds the quotes.
 *
 * @param StringVal the string to escape
 * @return the given string, escaped to produce a valid Json string.
 */
inline FString EscapeJsonString(const FString& StringVal)
{
	FString Result;
	return AppendEscapeJsonString(Result, StringVal);
}

/**
 * Template for Json writers.
 *
 * @param CharType The type of characters to print, i.e. TCHAR or ANSICHAR.
 * @param PrintPolicy The print policy to use when writing the output string (default = TPrettyJsonPrintPolicy).
 */
template <class CharType = TCHAR, class PrintPolicy = TPrettyJsonPrintPolicy<CharType> >
class TJsonWriter
{
public:

	static TSharedRef< TJsonWriter > Create( FArchive* const Stream, int32 InitialIndentLevel = 0 )
	{
		return MakeShareable( new TJsonWriter< CharType, PrintPolicy >( Stream, InitialIndentLevel ) );
	}

public:

	virtual ~TJsonWriter() = default;

	inline int32 GetIndentLevel() const { return IndentLevel; }

	bool CanWriteObjectStart() const
	{
		return CanWriteObjectWithoutIdentifier();
	}

	EJson GetCurrentElementType() const
	{
		return Stack.Num() > 0 ? Stack.Top() : EJson::None;
	}

	void WriteObjectStart()
	{
		check(CanWriteObjectWithoutIdentifier());
		WriteCommaIfNeeded();

		if (PreviousTokenWritten != EJsonToken::None )
		{
			PrintPolicy::WriteLineTerminator(Stream);
			PrintPolicy::WriteTabs(Stream, IndentLevel);
		}

		PrintPolicy::WriteChar(Stream, CharType('{'));
		++IndentLevel;
		Stack.Push( EJson::Object );
		PreviousTokenWritten = EJsonToken::CurlyOpen;
	}

	template<typename IdentifierType>
	void WriteObjectStart(IdentifierType&& Identifier)
	{
		check( Stack.Top() == EJson::Object );
		WriteIdentifier(Forward<IdentifierType>(Identifier));

		PrintPolicy::WriteLineTerminator(Stream);
		PrintPolicy::WriteTabs(Stream, IndentLevel);
		PrintPolicy::WriteChar(Stream, CharType('{'));
		++IndentLevel;
		Stack.Push( EJson::Object );
		PreviousTokenWritten = EJsonToken::CurlyOpen;
	}

	void WriteObjectEnd()
	{
		check( Stack.Top() == EJson::Object );

		PrintPolicy::WriteLineTerminator(Stream);

		--IndentLevel;
		PrintPolicy::WriteTabs(Stream, IndentLevel);
		PrintPolicy::WriteChar(Stream, CharType('}'));
		Stack.Pop();
		PreviousTokenWritten = EJsonToken::CurlyClose;
	}

	void WriteArrayStart()
	{
		check(CanWriteValueWithoutIdentifier());
		WriteCommaIfNeeded();

		if (PreviousTokenWritten != EJsonToken::None )
		{
			PrintPolicy::WriteLineTerminator(Stream);
			PrintPolicy::WriteTabs(Stream, IndentLevel);
		}

		PrintPolicy::WriteChar(Stream, CharType('['));
		++IndentLevel;
		Stack.Push( EJson::Array );
		PreviousTokenWritten = EJsonToken::SquareOpen;
	}

	template<typename IdentifierType>
	void WriteArrayStart(IdentifierType&& Identifier)
	{
		check( Stack.Top() == EJson::Object );
		WriteIdentifier(Forward<IdentifierType>(Identifier));

		PrintPolicy::WriteSpace( Stream );
		PrintPolicy::WriteChar(Stream, CharType('['));
		++IndentLevel;
		Stack.Push( EJson::Array );
		PreviousTokenWritten = EJsonToken::SquareOpen;
	}

	void WriteArrayEnd()
	{
		check( Stack.Top() == EJson::Array );

		--IndentLevel;

		if (PreviousTokenWritten != EJsonToken::SquareOpen)
		{
			PrintPolicy::WriteLineTerminator(Stream);
			PrintPolicy::WriteTabs(Stream, IndentLevel);
		}
		
		PrintPolicy::WriteChar(Stream, CharType(']'));
		Stack.Pop();
		PreviousTokenWritten = EJsonToken::SquareClose;
	}

	// Special case for bit fields
	void WriteValue(uint8 Value)
	{
		check(CanWriteValueWithoutIdentifier());
		WriteCommaIfNeeded();

		if (EJsonToken_IsShortValue(PreviousTokenWritten))
		{
			PrintPolicy::WriteSpace(Stream);
		}
		else
		{
			PrintPolicy::WriteLineTerminator(Stream);
			PrintPolicy::WriteTabs(Stream, IndentLevel);
		}

		PreviousTokenWritten = WriteValueOnly(Value);
	}

	template <class FValue>
	void WriteValue(FValue&& Value)
	{
		check(CanWriteValueWithoutIdentifier());
		WriteCommaIfNeeded();

		if (EJsonToken_IsShortValue(PreviousTokenWritten))
		{
			PrintPolicy::WriteSpace(Stream);
		}
		else
		{
			PrintPolicy::WriteLineTerminator(Stream);
			PrintPolicy::WriteTabs(Stream, IndentLevel);
		}

		PreviousTokenWritten = WriteValueOnly(Forward<FValue>(Value));
	}

	void WriteValue(FStringView Value)
	{
		check(CanWriteValueWithoutIdentifier());
		WriteCommaIfNeeded();

		PrintPolicy::WriteLineTerminator(Stream);
		PrintPolicy::WriteTabs(Stream, IndentLevel);
		PreviousTokenWritten = WriteValueOnly(Value);
	}

	void WriteValue(const FString& Value)
	{
		check(CanWriteValueWithoutIdentifier());
		WriteCommaIfNeeded();

		PrintPolicy::WriteLineTerminator(Stream);
		PrintPolicy::WriteTabs(Stream, IndentLevel);
		PreviousTokenWritten = WriteValueOnly(Value);
	}

	// Special case for bit fields
	template<typename IdentifierType>
	void WriteValue(IdentifierType&& Identifier, uint8 Value)
	{
		check( Stack.Top() == EJson::Object );
		WriteIdentifier(Forward<IdentifierType>(Identifier));

		PrintPolicy::WriteSpace(Stream);
		PreviousTokenWritten = WriteValueOnly(Value);
	}

	template<class FValue, typename IdentifierType>
	void WriteValue(IdentifierType&& Identifier, FValue&& Value)
	{
		check( Stack.Top() == EJson::Object );
		WriteIdentifier(Forward<IdentifierType>(Identifier));

		PrintPolicy::WriteSpace(Stream);
		PreviousTokenWritten = WriteValueOnly(Forward<FValue>(Value));
	}

	template<class ElementType, typename IdentifierType>
	void WriteValue(IdentifierType&& Identifier, const TArray<ElementType>& Array)
	{
		WriteArrayStart(Forward<IdentifierType>(Identifier));
		for (int Idx = 0; Idx < Array.Num(); Idx++)
		{
			WriteValue(Array[Idx]);
		}
		WriteArrayEnd();
	}

	template<typename IdentifierType, typename MapIdentifierType, class MapElementType>
	void WriteValue(IdentifierType&& Identifier, const TMap<MapIdentifierType, MapElementType>& Map)
	{
		WriteObjectStart(Forward<IdentifierType>(Identifier));
		for (const TPair<MapIdentifierType, MapElementType>& Element : Map)
		{
			WriteValue(Element.Key, Element.Value);
		}
		WriteObjectEnd();
	}

	template<typename MapIdentifierType, class MapElementType>
	void WriteValue(const TMap<MapIdentifierType, MapElementType>& Map)
	{
		WriteObjectStart();
		for (const TPair<MapIdentifierType, MapElementType>& Element : Map)
		{
			WriteValue(Element.Key, Element.Value);
		}
		WriteObjectEnd();
	}

	void WriteValue(FStringView Identifier, const TCHAR* Value)
	{
		WriteValue(Identifier, FStringView(Value));
	}

	// WARNING: THIS IS DANGEROUS. Use this only if you know for a fact that the Value is valid JSON!
	// Use this to insert the results of a different JSON Writer in.
	void WriteRawJSONValue( FStringView Identifier, FStringView Value )
	{
		WriteRawJSONValueImpl(Identifier, Value);
	}

	// WARNING: THIS IS DANGEROUS. Use this only if you know for a fact that the Value is valid JSON!
	// Use this to insert the results of a different JSON Writer in.
	void WriteRawJSONValue( FUtf8StringView Identifier, FUtf8StringView Value )
	{
		WriteRawJSONValueImpl(Identifier, Value);
	}

	template<typename IdentifierType>
	void WriteNull(IdentifierType&& Identifier)
	{
		WriteValue(Forward<IdentifierType>(Identifier), nullptr);
	}

	void WriteValue(const TCHAR* Value)
	{
		WriteValue(FStringView(Value));
	}

	// WARNING: THIS IS DANGEROUS. Use this only if you know for a fact that the Value is valid JSON!
	// Use this to insert the results of a different JSON Writer in.
	void WriteRawJSONValue(FStringView Value)
	{
		WriteRawJSONValueImpl(Value);
	}

	// WARNING: THIS IS DANGEROUS. Use this only if you know for a fact that the Value is valid JSON!
	// Use this to insert the results of a different JSON Writer in.
	void WriteRawJSONValue(FUtf8StringView Value)
	{
		WriteRawJSONValueImpl(Value);
	}

	void WriteNull()
	{
		WriteValue(nullptr);
	}

	virtual bool Close()
	{
		return ( PreviousTokenWritten == EJsonToken::None ||
				 PreviousTokenWritten == EJsonToken::CurlyClose  ||
				 PreviousTokenWritten == EJsonToken::SquareClose )
				&& Stack.Num() == 0;
	}

	/**
	 * WriteValue("Foo", Bar) should be equivalent to WriteIdentifierPrefix("Foo"), WriteValue(Bar)
	 */
	template<typename IdentifierType>
	void WriteIdentifierPrefix(IdentifierType&& Identifier)
	{
		check(Stack.Top() == EJson::Object);
		WriteIdentifier(Forward<IdentifierType>(Identifier));
		PrintPolicy::WriteSpace(Stream);
		PreviousTokenWritten = EJsonToken::Identifier;
	}

protected:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InStream An archive containing the input.
	 * @param InitialIndentLevel The initial indentation level.
	 */
	TJsonWriter( FArchive* const InStream, int32 InitialIndentLevel )
		: Stream( InStream )
		, Stack()
		, PreviousTokenWritten(EJsonToken::None)
		, IndentLevel(InitialIndentLevel)
	{ }

protected:

	inline bool CanWriteValueWithoutIdentifier() const
	{
		return Stack.Num() <= 0 || Stack.Top() == EJson::Array || PreviousTokenWritten == EJsonToken::Identifier;
	}

	inline bool CanWriteObjectWithoutIdentifier() const
	{
		return Stack.Num() <= 0 || Stack.Top() == EJson::Array || PreviousTokenWritten == EJsonToken::Identifier || PreviousTokenWritten == EJsonToken::Colon;
	}

	inline void WriteCommaIfNeeded()
	{
		if ( PreviousTokenWritten != EJsonToken::CurlyOpen && PreviousTokenWritten != EJsonToken::SquareOpen && PreviousTokenWritten != EJsonToken::Identifier && PreviousTokenWritten != EJsonToken::Comma && PreviousTokenWritten != EJsonToken::None)
		{
			PrintPolicy::WriteChar(Stream, CharType(','));

			PreviousTokenWritten = EJsonToken::Comma;
		}
	}

	template <typename InCharType>
	void WriteIdentifier(const InCharType* Identifier)
	{
		WriteCommaIfNeeded();
		PrintPolicy::WriteLineTerminator(Stream);

		PrintPolicy::WriteTabs(Stream, IndentLevel);
		WriteStringValue(TStringView<InCharType>(Identifier));
		PrintPolicy::WriteChar(Stream, CharType(':'));

		PreviousTokenWritten = EJsonToken::Identifier;
	}
	
	template <typename InCharType>
	void WriteIdentifier(TStringView<InCharType> Identifier)
	{
		WriteCommaIfNeeded();
		PrintPolicy::WriteLineTerminator(Stream);

		PrintPolicy::WriteTabs(Stream, IndentLevel);
		WriteStringValue(Identifier);
		PrintPolicy::WriteChar(Stream, CharType(':'));

		PreviousTokenWritten = EJsonToken::Identifier;
	}

	void WriteIdentifier(const FText& Identifier)
	{
		WriteIdentifier(Identifier.ToString()); // Does not copy
	}

	inline void WriteIdentifier(const FString& Identifier)
	{
		WriteCommaIfNeeded();
		PrintPolicy::WriteLineTerminator(Stream);

		PrintPolicy::WriteTabs(Stream, IndentLevel);
		WriteStringValue(FStringView(Identifier));
		PrintPolicy::WriteChar(Stream, CharType(':'));

		PreviousTokenWritten = EJsonToken::Identifier;
	}

	inline EJsonToken WriteValueOnly(bool Value)
	{
		PrintPolicy::WriteString(Stream, Value ? TEXTVIEW("true") : TEXTVIEW("false"));
		return Value ? EJsonToken::True : EJsonToken::False;
	}

	inline EJsonToken WriteValueOnly(float Value)
	{
		PrintPolicy::WriteFloat(Stream, Value);
		return EJsonToken::Number;
	}

	inline EJsonToken WriteValueOnly(double Value)
	{
		// Specify 17 significant digits, the most that can ever be useful from a double
		// In particular, this ensures large integers are written correctly
		PrintPolicy::WriteDouble(Stream, Value);
		return EJsonToken::Number;
	}

	inline EJsonToken WriteValueOnly(int32 Value)
	{
		return WriteValueOnly((int64)Value);
	}

	inline EJsonToken WriteValueOnly(int64 Value)
	{
		PrintPolicy::WriteString(Stream, WriteToString<32>(Value));
		return EJsonToken::Number;
	}

	// Special case for bit fields
	inline EJsonToken WriteValueOnly(uint8 Value)
	{
		return WriteValueOnly((uint64)Value);
	}

	inline EJsonToken WriteValueOnly(uint32 Value)
	{
		return WriteValueOnly((uint64)Value);
	}

	inline EJsonToken WriteValueOnly(uint64 Value)
	{
		PrintPolicy::WriteString(Stream, WriteToString<32>(Value));
		return EJsonToken::Number;
	}

	inline EJsonToken WriteValueOnly(TYPE_OF_NULLPTR)
	{
		PrintPolicy::WriteString(Stream, TEXTVIEW("null"));
		return EJsonToken::Null;
	}

	inline EJsonToken WriteValueOnly(const TCHAR* Value)
	{
		WriteStringValue(FStringView(Value));
		return EJsonToken::String;
	}

	inline EJsonToken WriteValueOnly(FStringView Value)
	{
		WriteStringValue(Value);
		return EJsonToken::String;
	}

	inline EJsonToken WriteValueOnly(FUtf8StringView Value)
	{
		WriteStringValue(Value);
		return EJsonToken::String;
	}

	template<typename ValueType>
	EJsonToken WriteValueOnly(TSharedRef<ValueType> ValueRef)
	{
		return WriteValueOnly(*ValueRef);
	}

	template<typename ValueType>
	EJsonToken WriteValueOnly(TSharedPtr<ValueType> ValuePtr)
	{
		if(ValuePtr.IsValid())
		{
			return WriteValueOnly(*ValuePtr);
		}
		else
		{
			return WriteValueOnly(nullptr);
		}
	}

	template<typename... ValueTypes>
	EJsonToken WriteValueOnly(const TVariant<ValueTypes...>& Variant)
	{
		return Visit([this](const auto& Value)
			{
				return WriteValueOnly(Value);
			}, Variant);
	}

	template<typename MapIdentifierType, typename MapElementType>
	EJsonToken WriteValueOnly(const TMap<MapIdentifierType, MapElementType>& Map)
	{
		WriteObjectStart();
		for (const TPair<MapIdentifierType, MapElementType>& Element : Map)
		{
			WriteValue(Element.Key, Element.Value);
		}
		WriteObjectEnd();

		return PreviousTokenWritten;
	}

	template<typename ArrayElementType>
	EJsonToken WriteValueOnly(const TArray<ArrayElementType>& Array)
	{
		WriteArrayStart();
		for (int Idx = 0; Idx < Array.Num(); Idx++)
		{
			WriteValue(Array[Idx]);
		}
		WriteArrayEnd();

		return PreviousTokenWritten;
	}

	virtual void WriteStringValue(FAnsiStringView String)
	{
		PrintPolicy::WriteChar(Stream, CharType('"'));
		WriteEscapedString(String);
		PrintPolicy::WriteChar(Stream, CharType('"'));
	}

	virtual void WriteStringValue(FStringView String)
	{
		PrintPolicy::WriteChar(Stream, CharType('"'));
		WriteEscapedString(String);
		PrintPolicy::WriteChar(Stream, CharType('"'));
	}

	virtual void WriteStringValue(const FString& String)
	{
		TJsonWriter::WriteStringValue(FStringView(String));
	}

	virtual void WriteStringValue(FUtf8StringView String)
	{
		PrintPolicy::WriteChar(Stream, CharType('"'));
		WriteEscapedString(String);
		PrintPolicy::WriteChar(Stream, CharType('"'));
	}

	virtual void WriteStringValue(const FUtf8String& String)
	{
		TJsonWriter::WriteStringValue(FUtf8StringView(String));
	}

	// WARNING: THIS IS DANGEROUS. Use this only if you know for a fact that the Value is valid JSON!
	// Use this to insert the results of a different JSON Writer in.
	template <typename InCharType>
	void WriteRawJSONValueImpl(FStringView Identifier, TStringView<InCharType> Value)
	{
		check(Stack.Top() == EJson::Object);
		WriteIdentifier(Identifier);

		PrintPolicy::WriteSpace(Stream);
		PrintPolicy::WriteString(Stream, Value);
		PreviousTokenWritten = EJsonToken::String;
	}

	template <typename InCharType>
	void WriteRawJSONValueImpl(TStringView<InCharType> Value)
	{
		check(CanWriteValueWithoutIdentifier());
		WriteCommaIfNeeded();

		if (PreviousTokenWritten != EJsonToken::True && PreviousTokenWritten != EJsonToken::False && PreviousTokenWritten != EJsonToken::SquareOpen)
		{
			PrintPolicy::WriteLineTerminator(Stream);
			PrintPolicy::WriteTabs(Stream, IndentLevel);
		}
		else
		{
			PrintPolicy::WriteSpace(Stream);
		}

		PrintPolicy::WriteString(Stream, Value);
		PreviousTokenWritten = EJsonToken::String;
	}

	template<typename InCharType>
	void WriteEscapedString(TStringView<InCharType> InView)
	{
		auto NeedsEscaping = [](InCharType Char) -> bool
		{
			switch (Char)
			{
			case CHARTEXT(InCharType, '\\'): return true;
			case CHARTEXT(InCharType, '\n'): return true;
			case CHARTEXT(InCharType, '\t'): return true;
			case CHARTEXT(InCharType, '\b'): return true;
			case CHARTEXT(InCharType, '\f'): return true;
			case CHARTEXT(InCharType, '\r'): return true;
			case CHARTEXT(InCharType, '\"'): return true;
			default:
				// Must escape control characters or non representable characters
				if (HasDestinationJsonStringCharRepresentation<CharType>(Char))
				{
					return false;
				}
				else
				{
					return true;
				}
			}
		};

		// Write successive runs of unescaped and escaped characters until the view is exhausted
		while (!InView.IsEmpty())
		{
			 // In case we are handed a very large string, avoid checking all of it at once without writing anything
			constexpr int32 LongestRun = 2048;
			int32 EndIndex = 0;
			for (; EndIndex < InView.Len() && EndIndex < LongestRun; ++EndIndex)
			{
				if (NeedsEscaping(InView[EndIndex]))
				{ 
					break;
				}
			}
			if (TStringView<InCharType> Blittable = InView.Left(EndIndex); !Blittable.IsEmpty())
			{
				PrintPolicy::WriteString(Stream, Blittable);
			}
			InView.RightChopInline(EndIndex);
			for (EndIndex = 0; EndIndex < InView.Len(); ++EndIndex)
			{
				InCharType Char = InView[EndIndex];
				switch (Char)
				{
				case CHARTEXT(InCharType, '\\'): PrintPolicy::WriteString(Stream, TEXTVIEW("\\\\")); continue;
				case CHARTEXT(InCharType, '\n'): PrintPolicy::WriteString(Stream, TEXTVIEW("\\n")); continue;
				case CHARTEXT(InCharType, '\t'): PrintPolicy::WriteString(Stream, TEXTVIEW("\\t")); continue;
				case CHARTEXT(InCharType, '\b'): PrintPolicy::WriteString(Stream, TEXTVIEW("\\b")); continue;
				case CHARTEXT(InCharType, '\f'): PrintPolicy::WriteString(Stream, TEXTVIEW("\\f")); continue;
				case CHARTEXT(InCharType, '\r'): PrintPolicy::WriteString(Stream, TEXTVIEW("\\r")); continue;
				case CHARTEXT(InCharType, '\"'): PrintPolicy::WriteString(Stream, TEXTVIEW("\\\"")); continue;
				default: break;
				}

				// Must escape control characters or non representable characters
				if (HasDestinationJsonStringCharRepresentation<CharType>(Char))
				{
					break;
				}
				else
				{
					TAnsiStringBuilder<8> Builder;
					Builder.Appendf("\\u%04x", Char);
					PrintPolicy::WriteString(Stream, Builder.ToView());
				}
			}
			InView.RightChopInline(EndIndex);
		}
	}

	FArchive* const Stream;
	TArray<EJson> Stack;
	EJsonToken PreviousTokenWritten;
	int32 IndentLevel;
};


template <class PrintPolicy = TPrettyJsonPrintPolicy<TCHAR>>
class TJsonStringWriter
	: public TJsonWriter<typename PrintPolicy::CharType, PrintPolicy>
{
public:
	using CharType = typename PrintPolicy::CharType;
	using StringType = TString<CharType>;

	static TSharedRef<TJsonStringWriter> Create(StringType* const InStream, int32 InitialIndent = 0 )
	{
		return MakeShareable(new TJsonStringWriter(InStream, InitialIndent));
	}

public:

	virtual ~TJsonStringWriter()
	{
		check(this->Stream->Close());
		delete this->Stream;
	}

	virtual bool Close() override
	{
		// This makes the slightly questionable assumption that the Bytes array does actually contain a string of contiguous CharType
		OutString->Reset(Bytes.Num() / sizeof(CharType));
		OutString->AppendChars((const CharType*)Bytes.GetData(), Bytes.Num() / sizeof(CharType));
		return TJsonWriter<CharType, PrintPolicy>::Close();
	}

protected:

	TJsonStringWriter(StringType* const InOutString, int32 InitialIndent )
		: TJsonWriter<CharType, PrintPolicy>(new FMemoryWriter(Bytes), InitialIndent)
		, Bytes()
		, OutString(InOutString)
	{ }

private:

	TArray<uint8> Bytes;
	StringType* OutString;
};

template <class CharType = TCHAR, class PrintPolicy = TPrettyJsonPrintPolicy<CharType>>
class TJsonWriterFactory
{
public:
	using StringType = TString<CharType>;

	static TSharedRef<TJsonWriter<CharType, PrintPolicy>> Create(FArchive* const Stream, int32 InitialIndent = 0)
	{
		return TJsonWriter< CharType, PrintPolicy >::Create(Stream, InitialIndent);
	}

	static TSharedRef<TJsonWriter<CharType, PrintPolicy>> Create(StringType* const Stream, int32 InitialIndent = 0)
	{
		return StaticCastSharedRef<TJsonWriter<CharType, PrintPolicy>>(TJsonStringWriter<PrintPolicy>::Create(Stream, InitialIndent));
	}
};
